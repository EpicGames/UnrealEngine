{
	"FileVersion": 3,
	"Version": 50,
	"VersionName": "2025.0.1",
	"FriendlyName": "Bridge",
	"Description": "Megascans Link for Quixel Bridge.",
	"Category": "Editor",
	"CreatedBy": "Quixel",
	"CreatedByURL": "https://www.quixel.se",
	"DocsURL": "https://help.quixel.com/hc/en-us/sections/360005846137-Quixel-Bridge-for-Unreal-Engine-5",
	"MarketplaceURL": "",
	"SupportURL": "support@quixel.com",
	"CanContainContent": true,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "Bridge",
			"Type": "Editor",
			"LoadingPhase": "PostDefault",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		},
		{
			"Name": "MegascansPlugin",
			"Type": "Editor",
			"LoadingPhase": "PostDefault",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "MetaHumanSDK",
			"Enabled": true
		}
	]
}
