[FilterPlugin]
/ThirdParty/Win/...