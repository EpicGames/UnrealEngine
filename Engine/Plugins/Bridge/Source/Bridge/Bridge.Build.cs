// Copyright Epic Games, Inc. All Rights Reserved.
namespace UnrealBuildTool.Rules
{
	using System.Collections.Generic;
	using UnrealBuildTool;
	using System.IO;
	using System;

	public class Bridge  : ModuleRules
	{
		private string ThirdPartyPath
		{
			get { return Path.GetFullPath(Path.Combine(ModuleDirectory, "../../ThirdParty/")); }
		}

		private void stageFiles(String[] FilesToStage)
		{
			foreach (var File in FilesToStage)
			{
				RuntimeDependencies.Add(File);
			}
		}

		public Bridge (ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Projects",
					"Core",
					"CoreUObject",
					"UnrealEd",
					"Engine",
					"InputCore",
					"RenderCore",
					"RHI",
					"Slate",
					"SlateCore",
					"UMG",
					"Json",
					"WebBrowser",
					"Networking",
					"Sockets",
					"ToolMenus",
					"ContentBrowserData",
					"PlacementMode",
					"MegascansPlugin",
					"MetaHumanSDKEditor",
					"ApplicationCore"
					});
			}
		}
}
