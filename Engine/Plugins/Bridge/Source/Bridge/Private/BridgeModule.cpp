// Copyright Epic Games, Inc. All Rights Reserved.
#include "IBridgeModule.h"
#include "UI/BridgeUIManager.h"
#include "NodeProcess.h"
#include "NodeProcessRunnableThread.h"

#define LOCTEXT_NAMESPACE "Bridge"

class FBridgeModule : public IBridgeModule
{
public:
	virtual void StartupModule() override
	{
		if (GIsEditor && !IsRunningCommandlet())
		{
			 FBridgeUIManager::Initialize();
		}
	}

	virtual void ShutdownModule() override
	{
		FBridgeUIManager::Shutdown();
		
		// TODO: Do some clean up (if required)
		TArray<NodeProcessRunnableThread*> NodeThreads = FNodeProcessManager::Get()->NodeThreads;
		for (int i = 0; i < NodeThreads.Num(); i++)
		{
			if (NodeThreads[i] != nullptr)
			{
				NodeThreads[i]->Exit();
			}
		}
	}
};

IMPLEMENT_MODULE(FBridgeModule, Bridge);

#undef LOCTEXT_NAMESPACE
