// Copyright Epic Games, Inc. All Rights Reserved.
#include "NodeCommManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NodeCommManager)

TSharedPtr<FNodeCommManager> FNodeCommManager::NodeCommManager;

TSharedPtr<FNodeCommManager> FNodeCommManager::Get()
{
	if (!NodeCommManager.IsValid())
	{
		NodeCommManager = MakeShareable(new FNodeCommManager);
	}
	return NodeCommManager;
}

void FNodeCommManager::NodeDataReceived(const FString& NodeData)
{
	UE_LOGF(LogTemp, Error, "Data received from Node process");
}

