// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "NodeCommManager.generated.h"

class FNodeCommManager
{
private:
	FNodeCommManager() = default;
	static TSharedPtr<FNodeCommManager> NodeCommManager;	

public:
	static TSharedPtr<FNodeCommManager> Get();
	void NodeDataReceived(const FString& NodeData);
	
};

//Dummy data structure, all json data to be converted to ustructs
USTRUCT()
struct FBifrostNodeInfo
{
	GENERATED_USTRUCT_BODY();
public:
	UPROPERTY(EditAnywhere,Category = "NodeData")
		FString Name;

	UPROPERTY(EditAnywhere, Category = "NodeData")
		FString Value;
};
