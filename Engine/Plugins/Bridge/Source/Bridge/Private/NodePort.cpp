// Copyright Epic Games, Inc. All Rights Reserved.
#include "NodePort.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NodePort)


UNodePort::UNodePort(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FString UNodePort::GetNodePort()
{
	FString NodePort;
	if (FFileHelper::LoadFileToString(NodePort, *PortFilePath))
	{
		return NodePort;
	}

	return TEXT("0");
}

bool UNodePort::IsNodeRunning()
{
	return true;
}

