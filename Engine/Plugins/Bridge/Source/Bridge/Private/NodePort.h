// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Misc/Paths.h"

#include "NodePort.generated.h"

UCLASS()
class UNodePort :
	public UObject
{
	GENERATED_UCLASS_BODY()
public:
	FString PortFilePath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Bridge"), TEXT("ThirdParty"), TEXT("node_port.txt"));
	bool bIsNodeRunning;
	UFUNCTION()
		FString GetNodePort();

	UFUNCTION()
		bool IsNodeRunning();
};
