// Copyright Epic Games, Inc. All Rights Reserved.
#include "NodeProcess.h"
#include "Misc/Paths.h"

#include "NodeProcessRunnableThread.h"
#include <iostream>
#include <string>

TSharedPtr<FNodeProcessManager> FNodeProcessManager::NodeProcessManager;

TSharedPtr<FNodeProcessManager> FNodeProcessManager::Get()
{
	if (!NodeProcessManager.IsValid())
	{
		NodeProcessManager = MakeShareable(new FNodeProcessManager);
	}
	return NodeProcessManager;
}

const FString FNodeProcessManager::GetPluginPath() const
{
	return FPaths::Combine(FPaths::EnginePluginsDir(), BridgePluginName);
}

const FString FNodeProcessManager::GetProcessURL() const
{
#if PLATFORM_WINDOWS
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(GetPluginPath(), TEXT("ThirdParty"), TEXT("Win"), TEXT("node-bifrost.exe")));
	return *MainFilePath;
#elif PLATFORM_MAC 
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(GetPluginPath(), TEXT("ThirdParty"), TEXT("Mac"), TEXT("node-bifrost.app")));
	return *MainFilePath;
#elif PLATFORM_LINUX
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(GetPluginPath(), TEXT("ThirdParty"), TEXT("Linux"), TEXT("node-bifrost")));
	return *MainFilePath;
#endif
}

void FNodeProcessManager::StartNodeProcess()
{
	const FString ProcessURL = GetProcessURL();

#if PLATFORM_WINDOWS || PLATFORM_LINUX
	NodeProcessHandle = FPlatformProcess::CreateProc(*ProcessURL, TEXT(""), true, true, true, &OutProcessId, 0, NULL, NULL, NULL);

	int32 ReturnCode;
	FPlatformProcess::GetProcReturnCode(NodeProcessHandle, &ReturnCode);
#elif PLATFORM_MAC
	NodeThreads.Add(new NodeProcessRunnableThread());
#endif
}

void FNodeProcessManager::RestartNodeProcess()
{
	NodeThreads.Add(new NodeProcessRunnableThread());
}
