// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HAL/PlatformProcess.h"
#include "Templates/SharedPointer.h"

class NodeProcessRunnableThread;

class FNodeProcessManager
{
private:
	FNodeProcessManager() = default;
	static TSharedPtr<FNodeProcessManager> NodeProcessManager;

	const FString BridgePluginName = TEXT("Bridge");
	uint32 OutProcessId = 0;
	FProcHandle NodeProcessHandle;

	const FString GetProcessURL() const;
	const FString GetPluginPath() const;	

public:
	static TSharedPtr<FNodeProcessManager> Get();
	TArray<NodeProcessRunnableThread*> NodeThreads;
	void StartNodeProcess();
	void RestartNodeProcess();
};

