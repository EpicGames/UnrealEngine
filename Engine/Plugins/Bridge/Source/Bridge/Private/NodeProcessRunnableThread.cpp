// Copyright Epic Games, Inc. All Rights Reserved.
#include "NodeProcessRunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Misc/Paths.h"

NodeProcessRunnableThread::NodeProcessRunnableThread()
{
	Thread = FRunnableThread::Create(this, TEXT("Bridge Background Service Thread"));
}

NodeProcessRunnableThread::~NodeProcessRunnableThread()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}
}

uint32 NodeProcessRunnableThread::Run()
{
	// Return success
	const FString BridgePluginName = TEXT("Bridge");
	const FString PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), BridgePluginName);
#if PLATFORM_WINDOWS
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("Win"), TEXT("node-bifrost.exe")));
#elif PLATFORM_MAC 
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("Mac"), TEXT("node-bifrost.app")));
#elif PLATFORM_LINUX
	FString MainFilePath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("Linux"), TEXT("node-bifrost")));
#endif

	FPlatformProcess::ExecElevatedProcess(*MainFilePath, NULL, NULL);

	return 0;
}
