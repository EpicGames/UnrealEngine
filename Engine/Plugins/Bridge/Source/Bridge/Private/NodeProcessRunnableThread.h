// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HAL/Runnable.h"

class FRunnableThread;

class NodeProcessRunnableThread : public FRunnable
{
	public:
		NodeProcessRunnableThread();
		virtual ~NodeProcessRunnableThread();

		// FRunnable functions
		virtual uint32 Run() override;

	protected:
		FRunnableThread* Thread = nullptr;
		bool bStopThread = false;
};
