// Copyright Epic Games, Inc. All Rights Reserved.
#include "UI/BridgeStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/SlateStyleRegistry.h"

TUniquePtr<FSlateStyleSet> FBridgeStyle::MSStyleInstance;

void FBridgeStyle::Initialize()
{
	if (!MSStyleInstance.IsValid())
	{
		MSStyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*MSStyleInstance);
	}
}

void FBridgeStyle::Shutdown()
{
	if (MSStyleInstance.IsValid())
	{
		
		FSlateStyleRegistry::UnRegisterSlateStyle(*MSStyleInstance);
		MSStyleInstance.Reset();
	}
}

FName FBridgeStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("BridgeStyle"));
	return StyleSetName;
}

FName FBridgeStyle::GetContextName()
{
	//static FName ContextName(TEXT("Megascans"));
	return FName(TEXT("Bridge"));
}

#define IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )
#define IMAGE_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush( Style->RootToContentDir(RelativePath, TEXT(".svg")), __VA_ARGS__)
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TUniquePtr< FSlateStyleSet > FBridgeStyle::Create()
{
	TUniquePtr< FSlateStyleSet > Style = MakeUnique<FSlateStyleSet>(GetStyleSetName());
	Style->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Bridge/Resources"));
	return Style;
}

void FBridgeStyle::SetIcon(const FString& StyleName, const FString& ResourcePath)
{
	FSlateStyleSet* Style = MSStyleInstance.Get();

	FString Name(GetContextName().ToString());
	Name = Name + "." + StyleName;
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon40x40));

	Name += ".Small";
	Style->Set(*Name, new IMAGE_BRUSH(ResourcePath, Icon20x20));
}

void FBridgeStyle::SetSVGIcon(const FString& StyleName, const FString& ResourcePath)
{
	FSlateStyleSet* Style = MSStyleInstance.Get();

	FString Name(GetContextName().ToString());
	Name = Name + "." + StyleName;
	Style->Set(*Name, new IMAGE_BRUSH_SVG(ResourcePath, Icon40x40));

	Name += ".Small";
	Style->Set(*Name, new IMAGE_BRUSH_SVG(ResourcePath, Icon20x20));
}

#undef IMAGE_BRUSH
#undef IMAGE_BRUSH_SVG

const ISlateStyle& FBridgeStyle::Get()
{
	check(MSStyleInstance);
	return *MSStyleInstance;
}
