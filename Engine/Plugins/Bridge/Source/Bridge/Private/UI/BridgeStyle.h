// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Templates/UniquePtr.h"

class FSlateStyleSet;
class ISlateStyle;
class FBridgeStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();
	static FName GetStyleSetName();
	static FName GetContextName();

	static void SetIcon(const FString& StyleName, const FString& ResourcePath);
	static void SetSVGIcon(const FString& StyleName, const FString& ResourcePath);
private:
	static TUniquePtr<FSlateStyleSet> Create();
	static TUniquePtr<FSlateStyleSet> MSStyleInstance;
};
