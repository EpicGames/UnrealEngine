// Copyright Epic Games, Inc. All Rights Reserved.
#include "UI/BridgeUIManager.h"
#include "Misc/FileHelper.h"
#include "UI/BridgeStyle.h"
#include "LevelEditor.h"
#include "NodePort.h"
#include "NodeProcess.h"

// WebBrowser
#include "SWebBrowser.h"
#include "Serialization/JsonSerializer.h"
#include "WebBrowserModule.h"
#include "IWebBrowserSingleton.h"
#include "IWebBrowserWindow.h"
// Widgets
#include "Framework/Application/SlateApplication.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ContentBrowserDataMenuContexts.h"
#include "UI/BrowserBinding.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "Bridge"
#define LEVELEDITOR_MODULE_NAME TEXT("LevelEditor")
#define CONTENTBROWSER_MODULE_NAME TEXT("ContentBrowser")

//#define ENABLE_BROWSER_DEV_TOOLS

TSharedPtr<FBridgeUIManagerImpl> FBridgeUIManager::Instance;
UBrowserBinding* FBridgeUIManager::BrowserBinding;

const FName BridgeTabName = "BridgeTab";

void FBridgeUIManager::Initialize()
{
	if (!Instance.IsValid())
	{
		//Instance = MakeUnique<FBridgeUIManagerImpl>();
		Instance = MakeShareable(new FBridgeUIManagerImpl);
		Instance->Initialize();
	}
}

void FBridgeUIManagerImpl::Initialize()
{
	FBridgeStyle::Initialize();
	SetupMenuItem();
}

void FBridgeUIManagerImpl::SetupMenuItem()
{
	FBridgeStyle::SetIcon("Logo", "Logo80x80");
	FBridgeStyle::SetIcon("ContextLogo", "Logo32x32");
	FBridgeStyle::SetSVGIcon("MenuLogo", "QuixelBridgeB");
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>(LEVELEDITOR_MODULE_NAME);
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	ToolbarExtender->AddToolBarExtension("Settings", EExtensionHook::After, nullptr, FToolBarExtensionDelegate::CreateRaw(this, &FBridgeUIManagerImpl::FillToolbar));
	LevelEditorModule.GetToolBarExtensibilityManager()->AddExtender(ToolbarExtender);

	// For Deleting Cookies
	// IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();
	// if (WebBrowserSingleton)
	// {
	// 	TSharedPtr<IWebBrowserCookieManager> CookieManager = WebBrowserSingleton->GetCookieManager();
	// 	if (CookieManager.IsValid())
	// 	{
	// 		CookieManager->DeleteCookies();
	// 	}
	// }

	// Adding Bridge entry to Quick Content menu.
	UToolMenu* AddMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu");
	FToolMenuSection& Section = AddMenu->FindOrAddSection("Content");
	Section.AddMenuEntry("OpenBridgeTab",
		LOCTEXT("OpenBridgeTab_Label", "Quixel Bridge"),
		LOCTEXT("OpenBridgeTab_Desc", "Opens the Quixel Bridge."),
		FSlateIcon(FBridgeStyle::GetStyleSetName(), "Bridge.MenuLogo"),
		FUIAction(FExecuteAction::CreateRaw(this, &FBridgeUIManagerImpl::CreateWindow), FCanExecuteAction())
	).InsertPosition = FToolMenuInsert("ImportContent", EToolMenuInsertType::After);


	UToolMenu* WindowMenu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Window");
	FToolMenuSection* ContentSectionPtr = WindowMenu->FindSection("GetContent");
	if (!ContentSectionPtr)
	{
		ContentSectionPtr = &WindowMenu->AddSection("GetContent", NSLOCTEXT("MainAppMenu", "GetContentHeader", "Get Content"));
	}
	ContentSectionPtr->AddMenuEntry("OpenBridgeTab",
		LOCTEXT("OpenBridgeTab_Label", "Quixel Bridge"),
		LOCTEXT("OpenBridgeTab_Desc", "Opens the Quixel Bridge."),
		FSlateIcon(FBridgeStyle::GetStyleSetName(), "Bridge.MenuLogo"),
		FUIAction(FExecuteAction::CreateRaw(this, &FBridgeUIManagerImpl::CreateWindow), FCanExecuteAction())
	);
	//Section.AddSeparator(NAME_None);

	//Adding Bridge entry to Content Browser context and New menu.
	UToolMenu* ContextMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AddNewContextMenu");
	//FToolMenuSection& ContextMenuSection = ContextMenu->AddSection("ContentBrowserMegascans", LOCTEXT("GetContentMenuHeading", "Quixel Content"));
	FToolMenuSection& ContextMenuSection = ContextMenu->FindOrAddSection("ContentBrowserGetContent");
	
	TWeakPtr<FBridgeUIManagerImpl> WeakPtr = AsShared();
	ContextMenuSection.AddDynamicEntry("GetMegascans", FNewToolMenuSectionDelegate::CreateLambda([WeakPtr](FToolMenuSection& InSection)
	{
		UContentBrowserDataMenuContext_AddNewMenu* AddNewMenuContext = InSection.FindContext<UContentBrowserDataMenuContext_AddNewMenu>();
		if (AddNewMenuContext && AddNewMenuContext->bCanBeModified && AddNewMenuContext->bContainsValidPackagePath && WeakPtr.IsValid())
		{
			InSection.AddMenuEntry(
				"GetMegascans",
				LOCTEXT("OpenBridgeTabText", "Add Quixel Content"),
				LOCTEXT("GetBridgeTooltip", "Add Megascans and DHI assets to project."),
				FSlateIcon(FBridgeStyle::GetStyleSetName(), "Bridge.MenuLogo"),
				FUIAction(FExecuteAction::CreateSP(WeakPtr.Pin().ToSharedRef(), &FBridgeUIManagerImpl::CreateWindow), FCanExecuteAction())
			);
		}
	}));

	/*TSharedPtr<FExtender> NewMenuExtender = MakeShareable(new FExtender);
	NewMenuExtender->AddMenuExtension("LevelEditor",
		EExtensionHook::After,
		NULL,
		FMenuExtensionDelegate::CreateRaw(this, &FBridgeUIManagerImpl::AddPluginMenu));
	LevelEditorModule.GetMenuExtensibilityManager()->AddExtender(NewMenuExtender);*/
	
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(BridgeTabName,
	FOnSpawnTab::CreateRaw(this, &FBridgeUIManagerImpl::CreateBridgeTab))
		.SetDisplayName(BridgeTabDisplay)
		.SetAutoGenerateMenuEntry(false)
		.SetTooltipText(BridgeToolTip);
}

void FBridgeUIManagerImpl::AddPluginMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("CustomMenu", TAttribute<FText>(FText::FromString("Quixel")));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OpenWindow", "Quixel Bridge"),
		LOCTEXT("ToolTip", "Open Quixel Bridge"),
		FSlateIcon(FBridgeStyle::GetStyleSetName(), "Bridge.Logo"),
		FUIAction(FExecuteAction::CreateRaw(this, &FBridgeUIManagerImpl::CreateWindow))
	);

	MenuBuilder.EndSection();
}

void FBridgeUIManagerImpl::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection(TEXT("QuixelBridge"));
	{
		ToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateRaw(this, &FBridgeUIManagerImpl::CreateWindow)),
			FName(TEXT("Quixel Bridge")),
			LOCTEXT("QMSLiveLink_label", "Bridge"),
			LOCTEXT("WorldProperties_ToolTipOverride", "Megascans Link with Bridge"),
			FSlateIcon(FBridgeStyle::GetStyleSetName(), "Bridge.Logo"),
			EUserInterfaceActionType::Button,
			FName(TEXT("QuixelBridge"))
		);
	}
	ToolbarBuilder.EndSection();
}

void FBridgeUIManagerImpl::CreateWindow()
{
	FGlobalTabmanager::Get()->TryInvokeTab(BridgeTabName);

	// Set desired window size (if the desired window size is less than main window size)
	// Rationale: the main window is mostly maximized - so the size is equal to screen size
	TArray<TSharedRef<SWindow>> Windows = FSlateApplication::Get().GetTopLevelWindows();
	if (Windows.Num() > 0)
	{
		FVector2D MainWindowSize = Windows[0]->GetSizeInScreen();
		float DesiredWidth = 1650;
		float DesiredHeight = 900;

		if (DesiredWidth < MainWindowSize.X && DesiredHeight < MainWindowSize.Y && LocalBrowserDock->GetParentWindow().IsValid())
		{
			// If Bridge is docked as a tab, the parent window will be the main window
			if (LocalBrowserDock->GetParentWindow() == Windows[0])
			{
				return;
			}

			LocalBrowserDock->GetParentWindow()->Resize(FVector2D(DesiredWidth, DesiredHeight));
			LocalBrowserDock->GetParentWindow()->MoveWindowTo(FVector2D((MainWindowSize.X - DesiredWidth) - 17, MainWindowSize.Y - DesiredHeight) / 2);
		}
	}
}

void FBridgeUIManager::Shutdown()
{
	if (FBridgeUIManager::Instance.IsValid())
	{
		if (FBridgeUIManager::Instance->WebBrowserWidget != NULL && FBridgeUIManager::Instance->WebBrowserWidget.IsValid())
		{
			FBridgeUIManager::Instance->WebBrowserWidget = NULL;
		}
		if (FBridgeUIManager::Instance->LocalBrowserDock != NULL && FBridgeUIManager::Instance->LocalBrowserDock.IsValid())
		{
			FBridgeUIManager::Instance->LocalBrowserDock = NULL;
		}
	}

	FBridgeStyle::Shutdown();
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(BridgeTabName);
}

TSharedRef<SDockTab> FBridgeUIManagerImpl::CreateBridgeTab(const FSpawnTabArgs& Args)
{
	IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		UE_LOGF(LogTemp, Error, "WebBrowserModule isn't available! Can't display Quixel Bridge tab contents.");
		return SAssignNew(LocalBrowserDock, SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0, 32, 0, 0))
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Text(FText::FromString("Unable to display the Quixel Bridge Browser"))
							.Justification(ETextJustify::Center)
					]
			];
	}

	// Start node process
	FNodeProcessManager::Get()->StartNodeProcess();

	// Delay launch on Mac to avoid getting "Background process stopped" toast
#if PLATFORM_MAC
	FGenericPlatformProcess::Sleep(1);
#endif

	FString PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Bridge"));
	FString IndexUrl = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("megascans"), TEXT("index.html")));
	FString TokenPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("megascans"), TEXT("token")));

	FString IndexFileContent;
	FFileHelper::LoadFileToString(IndexFileContent, *IndexUrl);

	FString Token;
	FFileHelper::LoadFileToString(Token, *TokenPath);

	FString FinalUrl;
	if (Token.Len() > 0)
	{
		TSharedPtr<FJsonObject> JsonParsed;
		TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(Token);
		if (FJsonSerializer::Deserialize(JsonReader, JsonParsed))
		{
			FString AccessToken = JsonParsed->GetStringField(TEXT("token"));
			FString RefreshToken = JsonParsed->GetStringField(TEXT("refreshToken"));
			const FString FileUrl = FPaths::Combine(TEXT("file:///"), IndexUrl);
			FinalUrl = FString::Printf(TEXT("%s?token=%s&refreshToken=%s"), *FileUrl, *AccessToken, *RefreshToken);
		}
	}
	else
	{
		FinalUrl = FPaths::Combine(TEXT("file:///"), IndexUrl);
	}

	FCreateBrowserWindowSettings WindowSettings;
	WindowSettings.InitialURL = FinalUrl;

#if PLATFORM_MAC
	WindowSettings.bMobileJSReturnInDict = false;
#endif

	IWebBrowserSingleton* WebBrowserSingleton = WebBrowserModule.GetSingleton();
	TSharedPtr<IWebBrowserWindow> Browser = WebBrowserSingleton->CreateBrowserWindow(WindowSettings);
	if (!Browser.IsValid())
	{
		UE_LOGF(LogTemp, Error, "WebBrowserWindow is invalid! Can't display Quixel Bridge tab contents.");
		return SAssignNew(LocalBrowserDock, SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FMargin(0, 32, 0, 0))
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
							.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Text(FText::FromString("Unable to display the Quixel Bridge Browser"))
							.Justification(ETextJustify::Center)
					]
			];
	}

	SAssignNew(WebBrowserWidget, SWebBrowser, Browser)
		.ShowAddressBar(false)
		.ShowControls(false);

#ifdef ENABLE_BROWSER_DEV_TOOLS
	WebBrowserSingleton->SetDevToolsShortcutEnabled(true);
	Browser->OnCreateWindow().BindLambda([](const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow, const TWeakPtr<IWebBrowserPopupFeatures>& PopupFeatures)
	{
		// Initialize a dialog
		auto DialogMainWindow = SNew(SWindow)
			.Title(FText::FromString(TEXT("Chrome Debugging Tools")))
			.ClientSize(FVector2D(700, 700))
			.SupportsMaximize(true)
			.SupportsMinimize(true)
			[
				SNew(SVerticalBox) +
				SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SWebBrowser, NewBrowserWindow.Pin())
					]
			];
		FSlateApplication::Get().AddWindow(DialogMainWindow);
		return true;
	});
#endif

	// by handling these keys, keys that were swallowed as Char events won't come out as unhandled KwyDown/Up events
	// that then go to the editor and change editor state (these arent needed if the function that calls these callbacks
	// was deleted, which may be the case soon)
	Browser->OnUnhandledKeyUp().BindLambda([](const FKeyEvent&) { return true; });
	Browser->OnUnhandledKeyDown().BindLambda([](const FKeyEvent&) { return true; });

	SAssignNew(LocalBrowserDock, SDockTab)
		.OnTabClosed_Lambda([](TSharedRef<class SDockTab> InParentTab)
		{
			// Kill node process if bound
			FBridgeUIManager::BrowserBinding->OnExitDelegate.ExecuteIfBound("Plugin Window Closed");
			FBridgeUIManager::BrowserBinding = NULL;

			// Clean up browser
			FBridgeUIManager::Instance->LocalBrowserDock = NULL;
			if (FBridgeUIManager::Instance->WebBrowserWidget.IsValid())
			{
				FBridgeUIManager::Instance->WebBrowserWidget.Reset();
			}
		})
		.TabRole(ETabRole::NomadTab)
		[
			WebBrowserWidget.ToSharedRef()
		];

	LocalBrowserDock->SetOnTabDraggedOverDockArea(
		FSimpleDelegate::CreateLambda([IndexUrl]()
										{
										FBridgeUIManager::Instance->WebBrowserWidget->Invalidate(EInvalidateWidgetReason::Layout);
										})
	);
	LocalBrowserDock->SetOnTabRelocated(
		FSimpleDelegate::CreateLambda([IndexUrl]()
										{
										FBridgeUIManager::Instance->WebBrowserWidget->Invalidate(EInvalidateWidgetReason::Layout);
										})
	);

	if (WebBrowserWidget.IsValid())
	{
		UNodePort* NodePortInfo = NewObject<UNodePort>();
		FBridgeUIManager::BrowserBinding = NewObject<UBrowserBinding>();
		FBridgeUIManager::Instance->WebBrowserWidget->BindUObject(TEXT("NodePortInfo"), NodePortInfo, true);
		FBridgeUIManager::Instance->WebBrowserWidget->BindUObject(TEXT("BrowserBinding"), FBridgeUIManager::BrowserBinding, true);
	}

	if (LocalBrowserDock.IsValid())
	{
		return LocalBrowserDock.ToSharedRef();
	}
	
	SAssignNew(LocalBrowserDock, SDockTab);

	return LocalBrowserDock.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
