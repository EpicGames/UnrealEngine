// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Text.h"

class FMenuBuilder;
class FSpawnTabArgs;
class FToolBarBuilder;
class SDockTab;
class SWebBrowser;
class SWindow;
class UBrowserBinding;


class FBridgeUIManagerImpl;
class FArguments;
class SCompoundWidget;

class FBridgeUIManagerImpl : public TSharedFromThis<FBridgeUIManagerImpl>
{
public:
	void Initialize();
	void Shutdown();
	void HandleBrowserUrlChanged(const FText& Url);
	TSharedPtr<SWebBrowser> WebBrowserWidget;
	TSharedPtr<SDockTab> LocalBrowserDock;
	TSharedPtr<SWindow> DragDropWindow;

private:
	void SetupMenuItem();
	void CreateWindow();
	TSharedRef<SDockTab> CreateBridgeTab(const FSpawnTabArgs& Args);
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);
	void AddPluginMenu(FMenuBuilder& MenuBuilder);

protected:
	const FText BridgeTabDisplay = FText::FromString("Bridge");
	const FText BridgeToolTip = FText::FromString("Launch Megascans Bridge");
};

class  FBridgeUIManager
{
public:
	static void Initialize();
	static void Shutdown();
	static TSharedPtr<FBridgeUIManagerImpl> Instance;
	static UBrowserBinding* BrowserBinding;
};

