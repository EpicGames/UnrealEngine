// Copyright Epic Games, Inc. All Rights Reserved.
#include "UI/BrowserBinding.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "Components/StaticMeshComponent.h"
#include "BridgeDragDropHelper.h"
#include "UI/BridgeUIManager.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Editor.h"
#include "WebBrowserModule.h"
#include "Engine/StaticMesh.h"
#include "IWebBrowserCookieManager.h"
#include "Misc/FileHelper.h"
#include "SMSWindow.h"
#include "Misc/Paths.h"
#include "TCPServer.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/DecalActor.h"
#include "Materials/MaterialInstanceConstant.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "NodeProcess.h"
#include "UnrealClient.h"
#include "SWebBrowser.h"
#include "WebJSFunction.h"
#include "Dom/JsonObject.h"
#include "Import/MetaHumanImport.h"
#include "ProjectUtilities/MetaHumanProjectUtilities.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BrowserBinding)

/**
* Drag drop action
**/
class FAssetDragDropCustomOp : public FAssetDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FAssetDragDropCustomOp, FAssetDragDropOp)

		// FDragDropOperation interface
		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
		virtual void OnDragged(const class FDragDropEvent& DragDropEvent) override;
		virtual void Construct() override;
		virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;
		// End of FDragDropOperation interface

		void SetCanDropHere(bool bCanDropHere)
		{
			MouseCursor = bCanDropHere ? EMouseCursor::TextEditBeam : EMouseCursor::SlashedCircle;
		}
		TArray<FString> ImageUrls;
		TArray<FString> IDs;
		static TSharedRef<FAssetDragDropCustomOp> New(TArray<FAssetData> AssetDataArray, UActorFactory* ActorFactory, TArray<FString> ImageUrls, TArray<FString> IDs);

	protected:
		FAssetDragDropCustomOp();
};

FAssetDragDropCustomOp::FAssetDragDropCustomOp()
{}

TSharedPtr<SWidget> FAssetDragDropCustomOp::GetDefaultDecorator() const
{
	TSharedPtr<SWebBrowser> PopupWebBrowser = SNew(SWebBrowser)
								   .ShowControls(false);

	FString ImageUrl = ImageUrls[0];
	int32 Count = ImageUrls.Num();

	// FBridgeUIManager::Instance->DragDropWindow = SNew(SWindow)
	//    .ClientSize(FVector2D(120, 120))
	//    .InitialOpacity(0.5f)
	//    .SupportsTransparency(EWindowTransparency::PerWindow)
	//    .CreateTitleBar(false)
	//    .HasCloseButton(false)
	//    .IsTopmostWindow(true)
	//    .FocusWhenFirstShown(false)
	//    .SupportsMaximize(false)
	//    .SupportsMinimize(false)
	//    [
	//        PopupWebBrowser.ToSharedRef()
	//    ];

	if (Count > 1)
	{
	   PopupWebBrowser->LoadString(FString::Printf(TEXT("<!DOCTYPE html><html lang=\"en\"> <head> <meta charset=\"UTF-8\"/> <meta name=\"viewport\" content=\"width=device-width, initial-scale=1.0\"/> <style>*{padding: 0px;}body{padding: 0px; margin: 0px;}#container{display: flex; position: relative; width: 100%%; height: 100%%; min-width: 120px; min-height: 120px; background: #202020; justify-content: center; align-items: center;}#full-image{max-width: 110px; max-height: 110px; display: block; font-size: 0;}#number-circle{position: absolute; border-radius: 50%%; width: 18px; height: 18px; padding: 4px; background: #fff; color: #666; text-align: center; font: 12px Arial, sans-serif; box-shadow: 1px 1px 1px #888888; opacity: 0.5;}</style> </head> <body> <div id=\"container\"> <img id=\"full-image\" src=\"%s\"/> <div id=\"number-circle\">+%d</div></div></body></html>"), *ImageUrl, Count-1), TEXT(""));
	}
	else
	{
	   PopupWebBrowser->LoadString(FString::Printf(TEXT("<!DOCTYPE html><html lang=\"en\"> <head> <meta charset=\"UTF-8\"/> <meta name=\"viewport\" content=\"width=device-width, initial-scale=1.0\"/> <style>*{padding: 0px;}body{padding: 0px; margin: 0px;}#container{display: flex; position: relative; width: 100%%; height: 100%%; min-width: 120px; min-height: 120px; background: #202020; justify-content: center; align-items: center;}#full-image{max-width: 110px; max-height: 110px; display: block; font-size: 0;}#number-circle{position: absolute; border-radius: 50%%; width: 18px; height: 18px; padding: 4px; background: #fff; color: #666; text-align: center; font: 16px Arial, sans-serif; box-shadow: 1px 1px 1px #888888; opacity: 0.5;}</style> </head> <body> <div id=\"container\"> <img id=\"full-image\" src=\"%s\"/></div></body></html>"), *ImageUrl), TEXT(""));
	}

	return SNew(SBox)
	[
			SNew(SBorder)
			[
				SNew(SBox)
				.HeightOverride(120)
				.WidthOverride(120)
				[
					PopupWebBrowser.ToSharedRef()
				]
			]
	];
}

void FAssetDragDropCustomOp::OnDragged(const class FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition());
	}
}


void FAssetDragDropCustomOp::Construct()
{
	MouseCursor = EMouseCursor::GrabHandClosed;

	FDragDropOperation::Construct();
}

void FAssetDragDropCustomOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	// UE_LOGF(LogTemp, Error, "On Drop Action");
	// UE_LOGF(LogTemp, Error, "bWasSwitchDragOp: %ls", FBridgeUIManager::BrowserBinding->bWasSwitchDragOperation ? TEXT("true") : TEXT("false"));

	FBridgeUIManager::BrowserBinding->bIsDragging = false;
	
	if (!bDropWasHandled)
	{
		if (FBridgeUIManager::BrowserBinding->bWasSwitchDragOperation) return;

		// UE_LOGF(LogTemp, Error, "Drag discarded");
		FBridgeUIManager::BrowserBinding->OnDropDiscardedDelegate.ExecuteIfBound(TEXT("dropped-discarded"));
		return;
	} 
}

TSharedRef<FAssetDragDropCustomOp> FAssetDragDropCustomOp::New(TArray<FAssetData> AssetDataArray, UActorFactory* ActorFactory, TArray<FString> ImageUrls, TArray<FString> IDs)
{
	// Create the drag-drop op containing the key
	TSharedRef<FAssetDragDropCustomOp> Operation = MakeShareable(new FAssetDragDropCustomOp);
	Operation->Init(AssetDataArray, TArray<FString>(), ActorFactory);
	Operation->ImageUrls = ImageUrls;
	Operation->IDs = IDs;
	Operation->Construct();

	return Operation;
}

///////////////////////////////////////////////////////////////////////////////////

class FBrowserBindingBulkImportHandler: public UE::MetaHuman::IMetaHumanBulkImportHandler
{
public:
  // MetaHumanIds is a list of the Quixel IDs of the MetaHumans to
  // be imported. This is an asynchronous operation. This function returns
  // immediately and the import operation that called it will immediately terminate.
  virtual void DoBulkImport(const TArray<FString>& MetaHumanIds) override
  {
	FBridgeUIManager::BrowserBinding->OnBulkExportMetahumansDelegate.ExecuteIfBound(MetaHumanIds);
  }

  static FBrowserBindingBulkImportHandler *Get()
  {
	static FBrowserBindingBulkImportHandler TheInstance;
	return &TheInstance;
  }
};

///////////////////////////////////////////////////////////////////////////////////

UBrowserBinding::UBrowserBinding(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
  UE::MetaHuman::FMetaHumanProjectUtilities::SetBulkImportHandler(FBrowserBindingBulkImportHandler::Get());
}

void UBrowserBinding::DialogSuccessCallback(FWebJSFunction DialogJSCallback)
{
	DialogSuccessDelegate.BindLambda(DialogJSCallback);
}

void UBrowserBinding::DialogFailCallback(FWebJSFunction DialogJSCallback)
{
	DialogFailDelegate.BindLambda(DialogJSCallback);
}

void UBrowserBinding::OnDroppedCallback(FWebJSFunction OnDroppedJSCallback)
{
	OnDroppedDelegate.BindLambda(OnDroppedJSCallback);
}

void UBrowserBinding::OnDropDiscardedCallback(FWebJSFunction OnDropDiscardedJSCallback)
{
	OnDropDiscardedDelegate.BindLambda(OnDropDiscardedJSCallback);
}

void UBrowserBinding::OnExitCallback(FWebJSFunction OnExitJSCallback)
{
	OnExitDelegate.BindLambda(OnExitJSCallback);
}

void UBrowserBinding::OnBulkExportMetahumansCallback(FWebJSFunction OnBulkExportMetahumansJSCallback)
{
	OnBulkExportMetahumansDelegate.BindLambda(OnBulkExportMetahumansJSCallback);
}

void UBrowserBinding::ShowDialog(FString Type, FString Url)
{
	TSharedPtr<SWebBrowser> MyWebBrowser;
	TSharedRef<SWebBrowser> MyWebBrowserRef = SAssignNew(MyWebBrowser, SWebBrowser)
		.InitialURL(Url)
		.ShowControls(false);

	MyWebBrowser->BindUObject(TEXT("BrowserBinding"), FBridgeUIManager::BrowserBinding, true);

	//Initialize a dialog
	DialogMainWindow = SNew(SWindow)
		.Title(FText::FromString(Type))
		.ClientSize(FVector2D(450, 700))
		.SupportsMaximize(false)
		.SupportsMinimize(false)		
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				MyWebBrowserRef
			]
		];
	
	FSlateApplication::Get().AddWindow(DialogMainWindow.ToSharedRef());
}

void UBrowserBinding::ShowLoginDialog(FString LoginUrl, FString ResponseCodeUrl) 
{
	// FString ProdUrl = TEXT("https://www.quixel.com/login?return_to=https%3A%2F%2Fquixel.com%2Fmegascans%2Fhome");
	// FString StagingUrl = TEXT("https://staging2.megascans.se/login?return_to=https%3A%2F%2Fstaging2.megascans.se%2Fmegascans%2Fhome");
	
	AsyncTask(ENamedThreads::Type::GameThread, [this, LoginUrl, ResponseCodeUrl]() 
	{
		TSharedRef<SWebBrowser> MyWebBrowserRef = SAssignNew(FBridgeUIManager::BrowserBinding->DialogMainBrowser, SWebBrowser)
						.InitialURL(LoginUrl)
						.ShowControls(false)
						.OnBeforePopup_Lambda([](FString NextUrl, FString Target)
						{
							FBridgeUIManager::BrowserBinding->DialogMainBrowser->LoadURL(NextUrl);
							return true;
						})
						.OnUrlChanged_Lambda([ResponseCodeUrl](const FText& Url) 
									{
										FString RedirectedUrl = Url.ToString();
										if (RedirectedUrl.StartsWith(ResponseCodeUrl))
										{
											// we are constructing this on the game thread, but this lambda is not necessairly 
											// running there and we need this to run on the GameThread because it interacts with
											// Slate.
											AsyncTask(ENamedThreads::Type::GameThread, [RedirectedUrl, ResponseCodeUrl] ()
										    {
												FBridgeUIManager::BrowserBinding->DialogMainWindow->RequestDestroyWindow();

												FString LoginCode = RedirectedUrl.Replace(*ResponseCodeUrl, TEXT(""));
												
												FBridgeUIManager::BrowserBinding->DialogSuccessDelegate.ExecuteIfBound("Login", LoginCode);
												
												FBridgeUIManager::BrowserBinding->DialogMainBrowser.Reset();
											});
										}
									}
						);
					
		//Initialize a dialog
		DialogMainWindow = SNew(SWindow)
			.Title(FText::FromString(TEXT("Login")))
			.ClientSize(FVector2D(450, 700))
			.SupportsMaximize(false)
#if PLATFORM_MAC && ALPHA_BLENDED_WINDOWS
			// this is needed to allow Slate-rendered Metal views to show up on XCode's view debugger
			// and for multi-view rendering to function properly
			.SupportsTransparency(EWindowTransparency::PerPixel)
#endif
			.SupportsMinimize(false)		
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[						
					MyWebBrowserRef
				]
			];

		FSlateApplication::Get().AddWindow(DialogMainWindow.ToSharedRef());
	});
}

FString UBrowserBinding::GetProjectPath()
{
	return FPaths::GetProjectFilePath();
}

void UBrowserBinding::SendSuccess(FString Value)
{
	FBridgeUIManager::BrowserBinding->DialogSuccessDelegate.ExecuteIfBound("Success", Value);
	DialogMainWindow->RequestDestroyWindow();
}

void UBrowserBinding::SaveAuthToken(FString Value)
{
	FString PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Bridge"));
	FString TokenPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("megascans"), TEXT("token")));
	FFileHelper::SaveStringToFile(Value, *TokenPath);
}

FString UBrowserBinding::GetAuthToken()
{
	FString PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Bridge"));
	FString TokenPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("megascans"), TEXT("token")));
	FString Cookie;
	FFileHelper::LoadFileToString(Cookie, *TokenPath);

	return Cookie;
}

void UBrowserBinding::SendFailure(FString Message)
{
	FBridgeUIManager::BrowserBinding->DialogSuccessDelegate.ExecuteIfBound("Failure", Message);
	DialogMainWindow->RequestDestroyWindow();
}

void UBrowserBinding::OpenExternalUrl(FString Url)
{
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
}

void UBrowserBinding::SwitchDragDropOp(TArray<FString> URLs, TSharedRef<FAssetDragDropOp> DragDropOperation)
{
	FBridgeUIManager::BrowserBinding->bWasSwitchDragOperation = true;
	FSlateApplication::Get().CancelDragDrop(); // calls onDrop method on the current drag drop operation
	FBridgeUIManager::BrowserBinding->bWasSwitchDragOperation = false;
	FBridgeUIManager::BrowserBinding->bIsDragging = true;

	const FVector2D CurrentCursorPosition = FSlateApplication::Get().GetCursorPos();
	const FVector2D LastCursorPosition = FSlateApplication::Get().GetLastCursorPos();

	TSet<FKey> PressedMouseButtons;
	PressedMouseButtons.Add(EKeys::LeftMouseButton);

	FModifierKeysState ModifierKeyState;


	const int32 UserIndexForMouse = FSlateApplication::Get().GetUserIndexForMouse();
	const uint32 CursorPointerIndex = FSlateApplicationBase::CursorPointerIndex;

	FPointerEvent FakePointerEvent(
			UserIndexForMouse,
			CursorPointerIndex,
			CurrentCursorPosition,
			LastCursorPosition,
			PressedMouseButtons,
			EKeys::Invalid,
			0,
			ModifierKeyState);

	if (TSharedPtr<SWindow> TopLevelWindow = FSlateApplication::Get().GetActiveTopLevelWindow(); TopLevelWindow.IsValid())
	{
		// Tell slate to enter drag and drop mode.
		// Make a fake mouse event for slate, so we can initiate a drag and drop.
		FDragDropEvent DragDropEvent(FakePointerEvent, DragDropOperation);

		FSlateApplication::Get().ProcessDragEnterEvent(TopLevelWindow.ToSharedRef(), DragDropEvent);
	}
}

void UBrowserBinding::OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors)
{
	if (bIsDragging)
	{
		// Take out 3d from the list of DroppedObjects
		TArray<UObject*> Filtered3dObjects;
		TArray<AActor*> Filtered3dActors;
		TArray<FString> Filtered3dIDs;

		for (int32 i = 0; i < DroppedObjects.Num(); i++)
		{
			if (DroppedObjects[i]->IsA(UStaticMesh::StaticClass()) || DroppedObjects[i]->IsA(UMaterialInstanceConstant::StaticClass()))
			{
				Filtered3dObjects.Add(DroppedObjects[i]);
			}
		}

		// Take out actors other than decals from the list of DroppedActors
		for (int32 i = 0; i < DroppedActors.Num(); i++)
		{
			if (DroppedActors[i]->IsA(AStaticMeshActor::StaticClass()) || DroppedActors[i]->IsA(ADecalActor::StaticClass()))
			{
				Filtered3dActors.Add(DroppedActors[i]);
			}
		}

		for (int32 i = 0; i < DragDropTypes.Num(); i++)
		{
			FString Type = DragDropTypes[i];
			FString ID = DragDropIDs[i];
			if (Type == "surface" || Type == "imperfection") continue;
			Filtered3dIDs.Add(ID);
		}

		if (Filtered3dActors.Num() != Filtered3dObjects.Num())
		{
			return;
		}

		// Search for the ID against filtered objects (specific case for decals)
		for (int32 i = 0; i < Filtered3dIDs.Num(); i++)
		{
			FString ID = Filtered3dIDs[i];
			for (int32 j = 0; j < Filtered3dObjects.Num(); j++)
			{
				UObject* Object = Filtered3dObjects[j];
				AActor* Actor = Filtered3dActors[j];
				if (Object != nullptr && Object->GetName().Find(ID) != -1)
				{
					AssetToSphereMap.Add(ID, Actor);
					Filtered3dIDs[i] = "-1";
					Filtered3dObjects[j] = nullptr;
					Filtered3dActors[j] = nullptr;
				}
			}
		}

		Filtered3dIDs = Filtered3dIDs.FilterByPredicate([](FString ID) { return ID != "-1"; });
		Filtered3dObjects = Filtered3dObjects.FilterByPredicate([](UObject* Object) { return Object != nullptr; });
		Filtered3dActors = Filtered3dActors.FilterByPredicate([](AActor* Actor) { return Actor != nullptr; });

		if (Filtered3dIDs.Num() != Filtered3dObjects.Num() || Filtered3dObjects.Num() != Filtered3dActors.Num())
		{
			return;
		}

		for (int32 i = 0; i < Filtered3dObjects.Num(); i++)
		{
			UObject* Object = Filtered3dObjects[i];
			AActor* Actor = Filtered3dActors[i];
			FString ID = Filtered3dIDs[i];

			if (Object->GetName().Find("Sphere") != -1)
			{
				AssetToSphereMap.Add(ID, Actor);
			}
		}
	}
}

void UBrowserBinding::OnApplyObjectToActor(UObject* DroppedObj, AActor* TargetActor)
{
	if (!bIsDragging)
	{
		return;
	}

	for (int32 i = 0; i < DragDropTypes.Num(); i++)
	{
		FString Type = DragDropTypes[i];
		if (Type == "surface" || Type == "imperfection")
		{
			if (FBridgeDragDropHelper::Instance->SurfaceToActorMap.Contains(DragDropIDs[i]))
			{
				FBridgeDragDropHelper::Instance->SurfaceToActorMap.Remove(DragDropIDs[i]);
			}

			FBridgeDragDropHelper::Instance->SurfaceToActorMap.Add(DragDropIDs[i], TargetActor);
		}
	}
}


void PopuplateInAssetData()
{
	TArray<FString> Types = FBridgeUIManager::BrowserBinding->DragDropTypes;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FString SurfaceInstancePath = TEXT("/Game/MSPresets/M_MS_Surface_Material/M_MS_Surface_Material.M_MS_Surface_Material");
	FAssetData SurfaceInstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(SurfaceInstancePath));

	for (int32 i = 0; i < Types.Num(); i++)
	{
		if (Types[i] == "surface" || Types[i] == "imperfection")
		{
			FBridgeUIManager::BrowserBinding->InAssetData.Add(SurfaceInstanceData);
		}
		else
		{
			// For 3d assets and decals - we start with a sphere
			FAssetData SphereData = FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicSphere.ToString()));
			FBridgeUIManager::BrowserBinding->InAssetData.Add(SphereData);
		}
	}
}

void HandleDecalHighInstance(FString AssetId, FAssetData AssetData)
{
	ADecalActor* Stage1Actor = Cast<ADecalActor>(FBridgeUIManager::BrowserBinding->AssetToSphereMap[AssetId]);
	if (!Stage1Actor) return;

	UWorld* CurrentWorld = GEngine->GetWorldContexts()[0].World();
	FTransform InitialTransform(Stage1Actor->GetActorLocation());	
	CurrentWorld->DestroyActor(Stage1Actor);
	FViewport* ActiveViewport = GEditor->GetActiveViewport();
	FEditorViewportClient* EditorViewClient = (FEditorViewportClient*)ActiveViewport->GetClient();
	ADecalActor* ADActor = Cast<ADecalActor>(CurrentWorld->SpawnActor(ADecalActor::StaticClass(), &InitialTransform));
	UMaterialInterface* MaterialInstance = Cast<UMaterialInterface>(AssetData.GetAsset());
	ADActor->SetDecalMaterial(MaterialInstance);
	ADActor->SetActorLabel(AssetData.AssetName.ToString());
	GEditor->EditorUpdateComponents();
	CurrentWorld->UpdateWorldComponents(true, false);
	ADActor->RerunConstructionScripts();
	GEditor->SelectActor(ADActor, true, false);

	FBridgeUIManager::BrowserBinding->AssetToSphereMap.Remove(AssetId);
}

bool IsAssetFoundInDragOperation(FString AssetId, TArray<FString> IDs)
{
	TArray<FString> AssetsInOperation = FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap[IDs[0]];
	for (int32 i = 0; i < AssetsInOperation.Num(); i++)
	{
		if (AssetsInOperation[i] == AssetId) return true;
	}

	UE_LOGF(LogTemp, Error, "AssetId not found in Drag operation: %ls", *AssetId);

	return false;
}

void FilterOutAssetInCurrentDragOperation(FString AssetId, TArray<FString> IDs, FString AssetType)
{
	// Don't filter out if the type is decal stage 1
	if (AssetType == "decal-stage-1") return;

	TArray<FString> AssetsInOperation = FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap[IDs[0]];
	// Filter out the asset we're currently processing
	TArray<FString> RemainingAssetsInOperation;
	// Iterate over AssetsInOperation
	for (int32 i = 0; i < AssetsInOperation.Num(); i++)
	{
		if (AssetsInOperation[i] != AssetId)
		{
			RemainingAssetsInOperation.Add(AssetsInOperation[i]);
		}
	}

	FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap[IDs[0]] = RemainingAssetsInOperation;
}

void RemoveSphereForAsset()
{
	for (int32 i = 0; i < FBridgeUIManager::BrowserBinding->InAssetData.Num(); i++)
	{
		if (FBridgeUIManager::BrowserBinding->InAssetData[i].GetObjectPathString().Contains("Sphere"))
		{
			FBridgeUIManager::BrowserBinding->InAssetData.RemoveAt(i);
			break;
		}
	}
}

void UBrowserBinding::DragStarted(TArray<FString> ImageUrls, TArray<FString> IDs, TArray<FString> Types)
{
	FBridgeUIManager::BrowserBinding->bWasSwitchDragOperation = false;
	FBridgeUIManager::BrowserBinding->bIsDragging = true;
	FBridgeUIManager::BrowserBinding->DragDropIDs = IDs;
	FBridgeUIManager::BrowserBinding->DragDropTypes = Types;

	if (!bIsDropEventBound)
	{
		FEditorDelegates::OnNewActorsDropped.AddUObject(FBridgeUIManager::BrowserBinding, &UBrowserBinding::OnNewActorsDropped);
		FEditorDelegates::OnApplyObjectToActor.AddUObject(FBridgeUIManager::BrowserBinding, &UBrowserBinding::OnApplyObjectToActor);

		FBridgeUIManager::BrowserBinding->bIsDropEventBound = true;
	}

	if (FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap.Contains(IDs[0]))
	{
		FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap.Remove(IDs[0]);
	}

	FBridgeUIManager::BrowserBinding->DragOperationToAssetsMap.Add(IDs[0], IDs);
	FBridgeUIManager::BrowserBinding->InAssetData.Empty();

	PopuplateInAssetData();
	UActorFactory* ActorFactory = GEditor->FindActorFactoryByClass(UActorFactoryStaticMesh::StaticClass());
	
	AsyncTask(ENamedThreads::Type::GameThread, [this, ImageUrls, IDs, ActorFactory] () 
	{
		TSharedRef<FAssetDragDropOp> DragDropOperation = FAssetDragDropCustomOp::New(FBridgeUIManager::BrowserBinding->InAssetData, ActorFactory, ImageUrls, IDs);
		SwitchDragDropOp(ImageUrls, DragDropOperation);

		// FBridgeUIManager::BrowserBinding->OnDroppedDelegate.ExecuteIfBound(TEXT("dropped"));
		FBridgeDragDropHelper::Instance->SetOnAddProgressiveStageData(FOnAddProgressiveStageDataCallbackInternal::CreateLambda(
			[this, ImageUrls, IDs](FAssetData AssetData, FString AssetId, FString AssetType, AStaticMeshActor* SpawnedActor) 
			{
				// UE_LOGF(LogTemp, Error, "Processing AssetId: %ls", *AssetId);
				if (AssetType == "decal-stage-4" && !FBridgeUIManager::BrowserBinding->bIsDragging)
				{
					HandleDecalHighInstance(AssetId, AssetData);
					return;
				}

				if (!IsAssetFoundInDragOperation(AssetId, IDs)) return;

				FilterOutAssetInCurrentDragOperation(AssetId, IDs, AssetType);

				// Remove the Sphere for this asset
				RemoveSphereForAsset();

				UWorld* CurrentWorld = GEngine->GetWorldContexts()[0].World();
				if (FBridgeUIManager::BrowserBinding->bIsDragging)
				{
					// We continue the d&d operation
					if (SpawnedActor == nullptr)
					{
						if (AssetType == "decal-stage-4")
						{
							// Loop over InAssetData
							for (int32 i = 0; i < FBridgeUIManager::BrowserBinding->InAssetData.Num(); i++)
							{
								if (FBridgeUIManager::BrowserBinding->InAssetData[i].GetObjectPathString().Contains(AssetId))
								{
									FBridgeUIManager::BrowserBinding->InAssetData.RemoveAt(i);
									break;
								}
							}
						}

						UActorFactory* ActorFactory = GEditor->FindActorFactoryByClass(UActorFactoryStaticMesh::StaticClass());
						FBridgeUIManager::BrowserBinding->InAssetData.Add(AssetData);
						TSharedRef<FAssetDragDropOp> DragDropOperation = FAssetDragDropCustomOp::New(FBridgeUIManager::BrowserBinding->InAssetData, ActorFactory, ImageUrls, IDs);
						SwitchDragDropOp(ImageUrls, DragDropOperation);
					}
					else
					{
						if (!FBridgeUIManager::BrowserBinding->AssetToSphereMap.Contains(AssetId)) return;

						AActor* FoundSphereActor = FBridgeUIManager::BrowserBinding->AssetToSphereMap[AssetId];
						if (FoundSphereActor == nullptr)
						{
							// UE_LOGF(LogTemp, Error, "FoundSphereActor is null");
							return;
						}

						FBridgeUIManager::BrowserBinding->AssetToSphereMap.Remove(AssetId);

						CurrentWorld->DestroyActor(FoundSphereActor);
					}
				}
				else
				{
					// Return if AssetToSphereMap is empty
					if (FBridgeUIManager::BrowserBinding->AssetToSphereMap.Num() == 0)
					{
						return;
					}

					// This is where we'd want to replace the individual cubes with actual assets
					// Find the sphere and replace it with the asset
					if (!FBridgeUIManager::BrowserBinding->AssetToSphereMap.Contains(AssetId)) return;

					FVector SpawnLocation;
					AActor* FoundSphereActor = FBridgeUIManager::BrowserBinding->AssetToSphereMap[AssetId];
					if (FoundSphereActor == nullptr) return;

					// Get the spawn location from sphere
					SpawnLocation = FoundSphereActor->GetActorLocation();
					FBridgeUIManager::BrowserBinding->AssetToSphereMap.Remove(AssetId);
					if (AssetType == "decal-stage-1" || AssetType == "decal-normal")
					{
						CurrentWorld->DestroyActor(FoundSphereActor);
						FViewport* ActiveViewport = GEditor->GetActiveViewport();
						FEditorViewportClient* EditorViewClient = (FEditorViewportClient*)ActiveViewport->GetClient();
						FTransform InitialTransform(SpawnLocation);	
						ADecalActor* ADActor = Cast<ADecalActor>(CurrentWorld->SpawnActor(ADecalActor::StaticClass(), &InitialTransform));
						UMaterialInterface* MaterialInstance = Cast<UMaterialInterface>(AssetData.GetAsset());
						ADActor->SetDecalMaterial(MaterialInstance);
						ADActor->SetActorLabel(AssetData.AssetName.ToString());
						GEditor->EditorUpdateComponents();
						CurrentWorld->UpdateWorldComponents(true, false);
						ADActor->RerunConstructionScripts();
						GEditor->SelectActor(ADActor, true, false);
						FBridgeUIManager::BrowserBinding->AssetToSphereMap.Add(AssetId, ADActor);
						return;
					}
					
					UStaticMesh* SourceMesh = Cast<UStaticMesh>(AssetData.GetAsset());
					CurrentWorld->DestroyActor(FoundSphereActor);
					FViewport* ActiveViewport = GEditor->GetActiveViewport();
					FEditorViewportClient* EditorViewClient = (FEditorViewportClient*)ActiveViewport->GetClient();
					FTransform InitialTransform(SpawnLocation);	
					AStaticMeshActor* SMActor;
					if (SpawnedActor == nullptr)
					{
						// UE_LOGF(LogTemp, Error, "SpawnedActor is null");
						SMActor = Cast<AStaticMeshActor>(CurrentWorld->SpawnActor(AStaticMeshActor::StaticClass(), &InitialTransform));
						SMActor->GetStaticMeshComponent()->SetStaticMesh(SourceMesh);
						SMActor->SetActorLabel(AssetData.AssetName.ToString());
					}
					else
					{
						SMActor = SpawnedActor;
						SMActor->SetActorTransform(InitialTransform);
						SMActor->SetActorLabel(AssetId);
					}

					GEditor->EditorUpdateComponents();
					CurrentWorld->UpdateWorldComponents(true, false);
					SMActor->RerunConstructionScripts();
					GEditor->SelectActor(SMActor, true, false);
				}
			}));
	});
}

void UBrowserBinding::Logout()
{
	// Delete Cookies
	IWebBrowserSingleton* WebBrowserSingleton = IWebBrowserModule::Get().GetSingleton();
	if (WebBrowserSingleton)
	{
		TSharedPtr<IWebBrowserCookieManager> CookieManager = WebBrowserSingleton->GetCookieManager();
		if (CookieManager.IsValid())
		{
			CookieManager->DeleteCookies();
		}
	}
	// Write file
	FString PluginPath = FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Bridge"));
	FString TokenPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("megascans"), TEXT("token")));
	FFileHelper::SaveStringToFile(TEXT(""), *TokenPath);
}

void UBrowserBinding::StartNodeProcess()
{
	// Start node process
	FNodeProcessManager::Get()->StartNodeProcess();
}

void UBrowserBinding::RestartNodeProcess()
{
	// Restart node process
	FNodeProcessManager::Get()->RestartNodeProcess();
}

void UBrowserBinding::OpenMegascansPluginSettings()
{
	MegascansSettingsWindow::OpenSettingsWindow();
}

void UBrowserBinding::ExportDataToMSPlugin(FString Data)
{
	FTCPServer::ImportQueue.Enqueue(Data);
}

