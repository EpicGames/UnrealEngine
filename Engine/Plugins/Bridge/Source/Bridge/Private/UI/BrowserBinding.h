// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once




#include "AssetRegistry/AssetData.h"
#include "BrowserBinding.generated.h"

class FAssetDragDropOp;
class SWebBrowser;
class SWindow;
struct FWebJSFunction;

DECLARE_DELEGATE_TwoParams(FOnDialogSuccess, FString, FString);
DECLARE_DELEGATE_TwoParams(FOnDialogFail, FString, FString);
DECLARE_DELEGATE_OneParam(FOnDropped, FString);
DECLARE_DELEGATE_OneParam(FOnDropDiscarded, FString);
DECLARE_DELEGATE_OneParam(FOnExit, FString);
DECLARE_DELEGATE_OneParam(FOnBulkExportMetahumans, TArray<FString>);

UCLASS()
class UBrowserBinding : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	void SwitchDragDropOp(TArray<FString> URLs, TSharedRef<FAssetDragDropOp> DragDropOperation);
	
	void OnNewActorsDropped(const TArray<UObject*>& DroppedObjects, const TArray<AActor*>& DroppedActors);
	void OnApplyObjectToActor(UObject* DroppedObj, AActor* TargetActor);

public:
	FOnDialogSuccess DialogSuccessDelegate;
	FOnDialogFail DialogFailDelegate;
	FOnDropped OnDroppedDelegate;
	FOnDropDiscarded OnDropDiscardedDelegate;
	FOnExit OnExitDelegate;
	FOnBulkExportMetahumans OnBulkExportMetahumansDelegate;

	TSharedPtr<SWindow> DialogMainWindow;
	TSharedPtr<SWebBrowser> DialogMainBrowser;

	UFUNCTION()
	void DialogSuccessCallback(FWebJSFunction DialogJSCallback);

	UFUNCTION()
	void DialogFailCallback(FWebJSFunction DialogJSCallback);

	UFUNCTION()
	void OnDroppedCallback(FWebJSFunction OnDroppedJSCallback);

	UFUNCTION()
	void OnDropDiscardedCallback(FWebJSFunction OnDropDiscardedJSCallback);

	UFUNCTION()
	void OnExitCallback(FWebJSFunction OnExitJSCallback);

	UFUNCTION()
	void OnBulkExportMetahumansCallback(FWebJSFunction OnBulkExportMetahumansJSCallback);

	UFUNCTION()
	void SaveAuthToken(FString Value);

	UFUNCTION()
	FString GetAuthToken();

	UFUNCTION()
	void SendSuccess(FString Value);

	UFUNCTION()
	void SendFailure(FString Message);

	UFUNCTION()
	void ShowDialog(FString Type, FString Url);

	UFUNCTION()
	void DragStarted(TArray<FString> ImageUrl, TArray<FString> IDs, TArray<FString> Types);

	UFUNCTION()
	void ShowLoginDialog(FString LoginUrl, FString ResponseCodeUrl);

	UFUNCTION()
	void OpenExternalUrl(FString Url);

	UFUNCTION()
	void ExportDataToMSPlugin(FString Data);

	UFUNCTION()
	FString GetProjectPath();

	UFUNCTION()
	void Logout();

	UFUNCTION()
	void StartNodeProcess();

	UFUNCTION()
	void RestartNodeProcess();

	UFUNCTION()
	void OpenMegascansPluginSettings();

	// Drag and drop

	bool bWasSwitchDragOperation = false;
	bool bIsDragging = false;
	bool bIsDropEventBound = false;

	TArray<FAssetData> InAssetData;
	TArray<FString> DragDropIDs;
	TArray<FString> DragDropTypes;
	TMap<FString, AActor*> AssetToSphereMap;
	TMap<FString, TArray<FString>> DragOperationToAssetsMap;
};
