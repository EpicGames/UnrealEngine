// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"

#define UE_API MEGASCANSPLUGIN_API

class AActor;
struct FAssetData;

class AStaticMeshActor;

DECLARE_DELEGATE_FourParams(FOnAddProgressiveStageDataCallbackInternal, FAssetData AssetData, FString AssetId, FString AssetType, AStaticMeshActor* SpawnedActor);

class FBridgeDragDropHelperImpl : public TSharedFromThis<FBridgeDragDropHelperImpl>
{
public:
    FOnAddProgressiveStageDataCallbackInternal OnAddProgressiveStageDataDelegate;
	TMap<FString, AActor*> SurfaceToActorMap;
    
    UE_API void SetOnAddProgressiveStageData(FOnAddProgressiveStageDataCallbackInternal InDelegate);
};

class FBridgeDragDropHelper
{
public:
	static UE_API void Initialize();
	static UE_API TSharedPtr<FBridgeDragDropHelperImpl> Instance;
};

#undef UE_API
