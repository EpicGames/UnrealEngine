// Copyright Epic Games, Inc. All Rights Reserved.
namespace UnrealBuildTool.Rules
{
	public class MegascansPlugin : ModuleRules
	{
		public MegascansPlugin(ReadOnlyTargetRules Target)
			: base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;


			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"EditorStyle",
					"Engine",
					"LevelEditor",
					"Settings",
					"Slate",
					"SlateCore",
					"Sockets",
					"Networking",
					"Json",
					"JsonUtilities",
					"MaterialEditor",
					"UnrealEd",
					"FoliageEdit",
					"Foliage",
					"HTTP",
					"StaticMeshEditor",
					"MeshBuilder",
					"TargetPlatform",
					"EditorScriptingUtilities",
					"Projects",
					"ApplicationCore",
					"MetaHumanSDKEditor"
				}
			);
		}
	}
}