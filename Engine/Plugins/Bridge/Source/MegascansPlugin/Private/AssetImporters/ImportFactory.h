// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "MSAssetImportData.h"

class FJsonObject;



class IAssetImportFactory
{
public :
	static TSharedPtr<IAssetImportFactory> CreateImporter(EAssetImportType ImporterType) ;
	virtual void ImportAsset(TSharedPtr<FJsonObject> AssetImportJson) = 0;

};

















