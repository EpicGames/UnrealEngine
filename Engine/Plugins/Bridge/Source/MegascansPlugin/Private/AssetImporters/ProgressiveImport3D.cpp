// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetImporters/ProgressiveImport3D.h"
#include "BridgeDragDropHelper.h"
#include "Utilities/MiscUtils.h"
#include "Engine/StaticMeshActor.h"
#include "Utilities/MaterialUtils.h"


#include "AssetRegistry/AssetRegistryModule.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/Paths.h"

#include "Engine/StreamableManager.h"
#include "Engine/AssetManager.h"

#include "EditorViewportClient.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"

#include "MaterialEditingLibrary.h"

#include "Async/Async.h"

#include "Kismet/GameplayStatics.h"
#include "Components/StaticMeshComponent.h"
#include "Misc/Paths.h"

#include "Modules/ModuleManager.h"
#include "StaticMeshCompiler.h"

TSharedPtr<FImportProgressive3D> FImportProgressive3D::ImportProgressive3DInst;

TSharedPtr<FImportProgressive3D> FImportProgressive3D::Get()
{
	if (!ImportProgressive3DInst.IsValid())
	{
		
		ImportProgressive3DInst = MakeShareable(new FImportProgressive3D);
	}
	return ImportProgressive3DInst;
}


void FImportProgressive3D::SpawnAtCenter(FAssetData AssetData, TSharedPtr<FUAssetData> ImportData, float LocationOffset, bool bIsNormal)
{
	FViewport* ActiveViewport = GEditor->GetActiveViewport();
	FEditorViewportClient* EditorViewClient = (FEditorViewportClient*)ActiveViewport->GetClient();

	FVector ViewPosition = EditorViewClient->GetViewLocation();
	FVector ViewDirection = EditorViewClient->GetViewRotation().Vector();
	FRotator InitialRotation(0.0f, 0.0f, 0.0f);

	FVector SpawnLocation = ViewPosition + (ViewDirection * 300.0f) ;
	SpawnLocation.X += LocationOffset;

	UWorld* CurrentWorld = GEngine->GetWorldContexts()[0].World();
	UStaticMesh* SourceMesh = Cast<UStaticMesh>(AssetData.GetAsset());
	FTransform InitialTransform(SpawnLocation);	
	

	AStaticMeshActor* SMActor = Cast<AStaticMeshActor>(CurrentWorld->SpawnActor(AStaticMeshActor::StaticClass(), &InitialTransform));
	SMActor->GetStaticMeshComponent()->SetStaticMesh(SourceMesh);

	SMActor->SetActorLabel(AssetData.AssetName.ToString());

	GEditor->EditorUpdateComponents();
	CurrentWorld->UpdateWorldComponents(true, false);
	SMActor->RerunConstructionScripts();

	GEditor->SelectActor(SMActor, true, false);

	if (bIsNormal) return;

	// if (!ProgressiveData.Contains(ImportData->AssetId) )
	// {
	// 	ProgressiveData.Add(ImportData->AssetId, SMActor);
	// }
}

void FImportProgressive3D::ImportAsset(TSharedPtr<FJsonObject> AssetImportJson, float LocationOffset, bool bIsNormal)
{

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FStreamableManager& Streamable = UAssetManager::GetStreamableManager();

	TSharedPtr<FUAssetData> ImportData = JsonUtils::ParseUassetJson(AssetImportJson);

	//FString UassetMetaString;
	//FFileHelper::LoadFileToString(UassetMetaString, *ImportData->ImportJsonPath);

	FUAssetMeta AssetMetaData = AssetUtils::GetAssetMetaData(ImportData->ImportJsonPath);
	//FJsonObjectConverter::JsonObjectStringToUStruct(UassetMetaString, &AssetMetaData);
	
	FString DestinationPath = AssetMetaData.assetRootPath;
	FString DestinationFolder = FPaths::Combine(FPaths::ProjectContentDir(), DestinationPath.Replace(TEXT("/Game/"), TEXT("")));
	

	

	if (AssetMetaData.assetType == TEXT("3dplant"))
	{
		CopyUassetFilesPlants(ImportData->FilePaths, DestinationFolder, AssetMetaData.assetTier);
	}
	else {
		CopyUassetFiles(ImportData->FilePaths, DestinationFolder);
	}

	if (bIsNormal)
	{
		FString MeshPath = AssetMetaData.meshList[0].path;
		FAssetData AssetMeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MeshPath));
		FSoftObjectPath ItemToStream = AssetMeshData.ToSoftObjectPath();
		if (!AssetMeshData.IsValid()) return;

		FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(AssetMeshData, ImportData->AssetId, AssetMetaData.assetType, nullptr);

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressive3D::HandleNormalAssetLoad, AssetMeshData, AssetMetaData, LocationOffset));

		return;
	}

	if (ImportData->ProgressiveStage != 1 && !PreviewDetails.Contains(ImportData->AssetId))
	{
		return;
	}
	

	if (!PreviewDetails.Contains(ImportData->AssetId))
	{
		TSharedPtr< FProgressiveData> ProgressiveDetails = MakeShareable(new FProgressiveData);
		PreviewDetails.Add(ImportData->AssetId, ProgressiveDetails);
	}


	if (ImportData->ProgressiveStage == 1)
	{		

		FString MeshPath = AssetMetaData.meshList[0].path;
		PreviewDetails[ImportData->AssetId]->PreviewMeshPath = MeshPath;
		FAssetData PreviewMeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MeshPath));
		FString MInstancePath = AssetMetaData.materialInstances[0].instancePath ;
		
		FAssetData MInstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MInstancePath));
		FSoftObjectPath ItemToStream = PreviewMeshData.ToSoftObjectPath();
		
		if (!MInstanceData.IsValid()) return;		

		// if (!ProgressiveData.Contains(ImportData->AssetId) )
		// {
		// 	ProgressiveData.Add(ImportData->AssetId, TEXT("Exists"));
		// }

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressive3D::HandlePreviewInstanceLoad, MInstanceData, ImportData->AssetId));

		FString AssetPath = PreviewMeshData.GetObjectPathString();
		FAssetData DraggedAssetData = FAssetData(LoadObject<UStaticMesh>(nullptr, *AssetPath));

		FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(DraggedAssetData, ImportData->AssetId, AssetMetaData.assetType, nullptr);
	}
	else if (ImportData->ProgressiveStage == 2)
	{
		FString AlbedoPath = TEXT("");
		FString TextureType = TEXT("albedo");

		for (FTexturesList TextureMeta : AssetMetaData.textureSets)
		{
			if (TextureMeta.type == TEXT("albedo"))
			{
				AlbedoPath = TextureMeta.path ;
			}
		}

		
		FAssetData AlbedoData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AlbedoPath));
		FSoftObjectPath ItemToStream = AlbedoData.ToSoftObjectPath();

		if (!AlbedoData.IsValid()) return;

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressive3D::HandlePreviewTextureLoad, AlbedoData, ImportData->AssetId, TextureType));
	}

	else if (ImportData->ProgressiveStage == 3)
	{

		FString NormalPath = TEXT("");
		FString TextureType = TEXT("normal");

		for (FTexturesList TextureMeta : AssetMetaData.textureSets)
		{
			if (TextureMeta.type == TEXT("normal"))
			{
				NormalPath = TextureMeta.path;
			}
		}
		
		
		FAssetData NormalData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(NormalPath));
		FSoftObjectPath ItemToStream = NormalData.ToSoftObjectPath();

		if (!NormalData.IsValid()) return;

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressive3D::HandlePreviewTextureLoad, NormalData, ImportData->AssetId, TextureType));

	}

	else if (ImportData->ProgressiveStage == 4)
	{
		bool bWaitNaniteConversion = false;
		FString MeshPath = AssetMetaData.meshList[0].path;

		FAssetData HighMeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MeshPath));
		if (ImportData->AssetType == TEXT("3d") && ImportData->AssetTier==0 && AssetMetaData.assetSubType == TEXT("singleMesh"))
		{
			bWaitNaniteConversion = true;
		}
		
		FSoftObjectPath ItemToStream = HighMeshData.ToSoftObjectPath();

		if (!HighMeshData.IsValid()) return;

		FAssetData PreviewMeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MeshPath));

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressive3D::HandleHighAssetLoad, HighMeshData, ImportData->AssetId, AssetMetaData, bWaitNaniteConversion));
	}
}

void FImportProgressive3D::HandlePreviewTextureLoad(FAssetData TextureData, FString AssetID,  FString Type)
{
	if (!IsValid(PreviewDetails[AssetID]->PreviewInstance))
	{
		return;
	}

	UTexture* PreviewTexture = Cast<UTexture>(TextureData.GetAsset());	

	UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(PreviewDetails[AssetID]->PreviewInstance, FName(*Type), PreviewTexture);
	AssetUtils::SavePackage(PreviewDetails[AssetID]->PreviewInstance);
}

void FImportProgressive3D::HandlePreviewInstanceLoad(FAssetData PreviewInstanceData, FString AssetID)
{
	
	PreviewDetails[AssetID]->PreviewInstance = Cast<UMaterialInstanceConstant>(PreviewInstanceData.GetAsset());
}



void FImportProgressive3D::HandleHighAssetLoad(FAssetData HighAssetData, FString AssetID, FUAssetMeta AssetMetaData, bool bWaitNaniteConversion)
{

	// if (!ProgressiveData.Contains(AssetID)) return;
	// AStaticMeshActor* PreviewActor = ProgressiveData[AssetID];
	UStaticMesh* SourceMesh = Cast<UStaticMesh>(HighAssetData.GetAsset());

	/*SourceMesh->OnPostMeshBuild().AddLambda([this](UStaticMesh* StaticMesh) {
		UE_LOGF(LogTemp, Error, "Data build complete...");
	});*/

	if (AssetMetaData.assetType != TEXT("3dplant"))
	{
		AssetUtils::ConvertToVT(AssetMetaData);
	}

	if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
	{
		AssetUtils::DeleteAsset(AssetMetaData.materialInstances[0].instancePath);
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
		
	}
	else if (AssetUtils::IsVTEnabled() && AssetMetaData.assetType != TEXT("3dplant"))
	{
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}

	AssetUtils::ManageImportSettings(AssetMetaData);

	AsyncTask(ENamedThreads::AnyThread, [this, HighAssetData, AssetID, AssetMetaData, bWaitNaniteConversion]() {
		AsyncCacheData(HighAssetData, AssetID, AssetMetaData, bWaitNaniteConversion);
	});
}



void FImportProgressive3D::AsyncCacheData(FAssetData HighAssetData, FString AssetID, FUAssetMeta AssetMetaData, bool bWaitNaniteConversion)
{

	UStaticMesh* SourceMesh = Cast<UStaticMesh>(HighAssetData.GetAsset());

	if (FStaticMeshCompilingManager::Get().IsAsyncCompilationAllowed(SourceMesh) && FStaticMeshCompilingManager::Get().IsAsyncStaticMeshCompilationEnabled())
	{
		while (SourceMesh->IsCompiling())
		{
			FPlatformProcess::Sleep(1.0f);
		}
	}


	AsyncTask(ENamedThreads::GameThread, [this, HighAssetData, AssetID]() {
		SwitchHigh(HighAssetData, AssetID);
	});	

}

void FImportProgressive3D::SwitchHigh(FAssetData HighAssetData, FString AssetID)
{
	TArray<AActor*> FoundActors;
	UGameplayStatics::GetAllActorsOfClass(GEngine->GetWorldContexts()[0].World(), AStaticMeshActor::StaticClass(), FoundActors);
	
	UStaticMesh* SourceMesh = Cast<UStaticMesh>(HighAssetData.GetAsset());

	if (!IsValid(SourceMesh)) return;

	/*AStaticMeshActor* PreviewActor = ProgressiveData[AssetID];
	if (PreviewActor != nullptr) {
		PreviewActor->GetStaticMeshComponent()->SetStaticMesh(SourceMesh);
	}*/

	for (AActor* StaticMeshActor : FoundActors)
	{
		TArray<UStaticMeshComponent*> MeshComponents;
		StaticMeshActor->GetComponents(MeshComponents);
		UStaticMesh* InstanceMesh = MeshComponents[0]->GetStaticMesh();

		if (InstanceMesh->GetPathName() == PreviewDetails[AssetID]->PreviewMeshPath)
		{
			MeshComponents[0]->SetStaticMesh(Cast<UStaticMesh>(SourceMesh));
		}
	}

	/*FString AssetPathHigh = FPaths::GetPath(SourceMesh->GetPathName());
	FString PreviwPath = FPaths::Combine(AssetPathHigh, TEXT("Preview"));*/
	//AssetUtils::DeleteDirectory(PreviwPath);

	
	// ProgressiveData.Remove(AssetID);
	PreviewDetails.Remove(AssetID);
}




// Handle normal drag and drop import.
void FImportProgressive3D::HandleNormalAssetLoad(FAssetData NormalAssetData, FUAssetMeta AssetMetaData, float LocationOffset)
{
	UStaticMesh* SourceMesh = Cast<UStaticMesh>(NormalAssetData.GetAsset());

	if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
	{
		AssetUtils::DeleteAsset(AssetMetaData.materialInstances[0].instancePath);
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}

	AssetUtils::ManageImportSettings(AssetMetaData);

	AsyncTask(ENamedThreads::AnyThread, [this, NormalAssetData, AssetMetaData, LocationOffset]() {
		AsyncNormalImportCache(NormalAssetData, AssetMetaData, LocationOffset);
	});

}


void FImportProgressive3D::AsyncNormalImportCache(FAssetData NormalAssetData, FUAssetMeta AssetMetaData, float LocationOffset)
{
	UStaticMesh* SourceMesh = Cast<UStaticMesh>(NormalAssetData.GetAsset());

	if (FStaticMeshCompilingManager::Get().IsAsyncCompilationAllowed(SourceMesh) && FStaticMeshCompilingManager::Get().IsAsyncStaticMeshCompilationEnabled())
	{
		while (SourceMesh->IsCompiling())
		{
			FPlatformProcess::Sleep(1.0f);
		}
	}	

}
