// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Templates/SharedPointer.h"

class AStaticMeshActor;
class FJsonObject;
class UMaterialInstanceConstant;
struct FAssetData;
struct FUAssetData;
struct FUAssetMeta;

struct FProgressiveData
{
	AStaticMeshActor* ActorInLevel;
	UMaterialInstanceConstant* PreviewInstance;
	FString PreviewFolderPath;
	FString PreviewMeshPath;
};

class FImportProgressive3D
{

private:
	FImportProgressive3D() = default;
	static TSharedPtr<FImportProgressive3D> ImportProgressive3DInst;
	void SpawnAtCenter(FAssetData AssetData, TSharedPtr<FUAssetData> ImportData, float LocationOffset, bool bIsNormal= false);
	// TMap<FString, FString> ProgressiveData;
	
	TMap<FString, TSharedPtr<FProgressiveData>> PreviewDetails;
	

	void AsyncCacheData(FAssetData HighAssetData, FString AssetID, FUAssetMeta AssetMetaData, bool bWaitNaniteConversion=false);
	void SwitchHigh(FAssetData HighAssetData, FString AssetID);

	void AsyncNormalImportCache(FAssetData HighAssetData, FUAssetMeta AssetMetaData, float LocationOffset);
	

public:
	static TSharedPtr<FImportProgressive3D> Get();
	void ImportAsset(TSharedPtr<FJsonObject> AssetImportJson, float LocationOffset, bool bIsNormal = false);

	void HandlePreviewTextureLoad(FAssetData TextureData, FString AssetID, FString Type);
	void HandlePreviewInstanceLoad(FAssetData PreviewInstanceData, FString AssetID);
	
	void HandleHighAssetLoad(FAssetData HighAssetData, FString AssetID, FUAssetMeta AssetMetaData, bool bWaitNaniteConversion = false);

	void HandleNormalAssetLoad(FAssetData NormalAssetData , FUAssetMeta AssetMetaData, float LocationOffset);
	
};
