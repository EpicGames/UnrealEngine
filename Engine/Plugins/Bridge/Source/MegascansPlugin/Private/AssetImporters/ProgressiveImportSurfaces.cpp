// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetImporters/ProgressiveImportSurfaces.h"
#include "BridgeDragDropHelper.h"
#include "Utilities/MiscUtils.h"
#include "Engine/StaticMeshActor.h"
#include "Utilities/MaterialUtils.h"
#include "MSSettings.h"


#include "AssetRegistry/AssetRegistryModule.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/Paths.h"

#include "Engine/StreamableManager.h"
#include "Engine/AssetManager.h"

#include "EditorViewportClient.h"
#include "Engine/Texture.h"

#include "MaterialEditingLibrary.h"


#include "Components/StaticMeshComponent.h"
#include "Modules/ModuleManager.h"




TSharedPtr<FImportProgressiveSurfaces> FImportProgressiveSurfaces::ImportProgressiveSurfacesInst;



TSharedPtr<FImportProgressiveSurfaces> FImportProgressiveSurfaces::Get()
{
	if (!ImportProgressiveSurfacesInst.IsValid())
	{
		ImportProgressiveSurfacesInst = MakeShareable(new FImportProgressiveSurfaces);
	}
	return ImportProgressiveSurfacesInst;
}


void FImportProgressiveSurfaces::ImportAsset(TSharedPtr<FJsonObject> AssetImportJson, float LocationOffset, bool bIsNormal)
{

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FStreamableManager& Streamable = UAssetManager::GetStreamableManager();

	TSharedPtr<FUAssetData> ImportData = JsonUtils::ParseUassetJson(AssetImportJson);

	/*FString UassetMetaString;
	FFileHelper::LoadFileToString(UassetMetaString, *ImportData->ImportJsonPath);*/

	FUAssetMeta AssetMetaData = AssetUtils::GetAssetMetaData(ImportData->ImportJsonPath);
	//FJsonObjectConverter::JsonObjectStringToUStruct(UassetMetaString, &AssetMetaData);

	FString DestinationPath = AssetMetaData.assetRootPath;
	FString DestinationFolder = FPaths::Combine(FPaths::ProjectContentDir(), DestinationPath.Replace(TEXT("/Game/"), TEXT("")));

	CopyUassetFiles(ImportData->FilePaths, DestinationFolder);

	if (AssetMetaData.assetType == TEXT("atlas"))
	{
		if (bIsNormal)
		{
			FString InstancePath = AssetMetaData.materialInstances[0].instancePath;
			FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(InstancePath));
			FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(AssetData, ImportData->AssetId, "decal-normal", nullptr);
			return;
		}

		if (!PreviewDetails.Contains(ImportData->AssetId))
		{
			TSharedPtr< FProgressiveSurfaces> ProgressiveDetails = MakeShareable(new FProgressiveSurfaces);
			PreviewDetails.Add(ImportData->AssetId, ProgressiveDetails);
		}

		if (ImportData->ProgressiveStage == 1)
		{
			FString InstancePath = AssetMetaData.materialInstances[0].instancePath;
			FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(InstancePath));
			FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(AssetData, ImportData->AssetId, TEXT("decal-stage-1"), nullptr);
		}
		if (ImportData->ProgressiveStage == 4)
		{
			 FString MInstanceHighPath = AssetMetaData.materialInstances[0].instancePath;
			 FAssetData MInstanceHighData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MInstanceHighPath));
			 FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(MInstanceHighData, ImportData->AssetId, TEXT("decal-stage-4"), nullptr);
		}

		return;
	}

	if (bIsNormal)
	{
		FString MInstancePath = AssetMetaData.materialInstances[0].instancePath;
		FAssetData MInstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MInstancePath));

		if (!MInstanceData.IsValid()) return;

		FSoftObjectPath ItemToStream = MInstanceData.ToSoftObjectPath();
		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressiveSurfaces::HandleNormalMaterialLoad, MInstanceData, AssetMetaData,  LocationOffset));

		return;
	}

	if (AssetMetaData.assetSubType == TEXT("imperfection") && ImportData->ProgressiveStage == 3)
	{
		ImportData->ProgressiveStage = 4;
	}
	
	if (ImportData->ProgressiveStage != 1 && !PreviewDetails.Contains(ImportData->AssetId))
	{
		return;
	}

	if (!PreviewDetails.Contains(ImportData->AssetId) )
	{
		TSharedPtr< FProgressiveSurfaces> ProgressiveDetails = MakeShareable(new FProgressiveSurfaces);
		PreviewDetails.Add(ImportData->AssetId, ProgressiveDetails);
	}

	if (ImportData->ProgressiveStage == 1)
	{
		FString MInstancePath = AssetMetaData.materialInstances[0].instancePath;
		FAssetData MInstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MInstancePath));

		if (!MInstanceData.IsValid())
		{
			PreviewDetails[ImportData->AssetId]->PreviewInstance = nullptr;
			return;
		}

		FSoftObjectPath ItemToStream = MInstanceData.ToSoftObjectPath();

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressiveSurfaces::HandlePreviewInstanceLoad, MInstanceData, ImportData->AssetId, LocationOffset));
	}
	else if (ImportData->ProgressiveStage == 2)
	{
		FString TexturePath = TEXT("");
		FString TextureType = TEXT("");
		if (AssetMetaData.assetSubType == TEXT("imperfection"))
		{
			TextureType = TEXT("roughness");
		}
		else
		{
			TextureType = TEXT("albedo");
		}

		for (FTexturesList TextureMeta : AssetMetaData.textureSets)
		{
			if (TextureMeta.type == TextureType)
			{
				TexturePath = TextureMeta.path;
			}
		}		

		FAssetData TextureData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(TexturePath));

		if (!TextureData.IsValid()) return;

		FSoftObjectPath ItemToStream = TextureData.ToSoftObjectPath();
		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressiveSurfaces::HandlePreviewTextureLoad, TextureData, ImportData->AssetId, TextureType));
	}

	else if (ImportData->ProgressiveStage == 3)
	{
		FString NormalPath = TEXT("");
		FString TextureType = TEXT("normal");

		for (FTexturesList TextureMeta : AssetMetaData.textureSets)
		{
			if (TextureMeta.type == TextureType)
			{
				NormalPath = TextureMeta.path;
			}
		}		

		FAssetData NormalData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(NormalPath));

		if (!NormalData.IsValid()) return;

		FSoftObjectPath ItemToStream = NormalData.ToSoftObjectPath();
		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressiveSurfaces::HandlePreviewTextureLoad, NormalData, ImportData->AssetId, TextureType));
	}

	else if (ImportData->ProgressiveStage == 4)
	{	
		FString MInstanceHighPath = AssetMetaData.materialInstances[0].instancePath;
		FAssetData MInstanceHighData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MInstanceHighPath));

		if (!MInstanceHighData.IsValid()) return;

		FSoftObjectPath ItemToStream = MInstanceHighData.ToSoftObjectPath();

		Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FImportProgressiveSurfaces::HandleHighInstanceLoad, MInstanceHighData, ImportData->AssetId, AssetMetaData));
	}
}

void FImportProgressiveSurfaces::HandlePreviewTextureLoad(FAssetData TextureData, FString AssetID, FString Type)
{
	if (!PreviewDetails.Contains(AssetID) || !IsValid(PreviewDetails[AssetID]->PreviewInstance))
	{
		return;
	}

	UTexture* PreviewTexture = Cast<UTexture>(TextureData.GetAsset());
	UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(PreviewDetails[AssetID]->PreviewInstance, FName(*Type), PreviewTexture);
	AssetUtils::SavePackage(PreviewDetails[AssetID]->PreviewInstance);
}

void FImportProgressiveSurfaces::HandlePreviewInstanceLoad(FAssetData PreviewInstanceData, FString AssetID, float LocationOffset)
{
	PreviewDetails[AssetID]->PreviewInstance = Cast<UMaterialInstanceConstant>(PreviewInstanceData.GetAsset());
	SpawnMaterialPreviewActor(AssetID, LocationOffset);
}

void FImportProgressiveSurfaces::SpawnMaterialPreviewActor(FString AssetID, float LocationOffset, bool bIsNormal, FAssetData MInstanceData)
{
	const UMegascansSettings* MegascansSettings = GetDefault<UMegascansSettings>();

	if (MegascansSettings->bApplyToSelection)
	{
		if (bIsNormal)
		{
			FMaterialUtils::ApplyMaterialToSelection(MInstanceData.GetPackage()->GetPathName());
			//PreviewDetails[AssetID]->ActorsInLevel = FMaterialUtils::ApplyMaterialToSelection(MInstanceData.GetPackage()->GetPathName());
		}
		else
		{
			PreviewDetails[AssetID]->ActorsInLevel = FMaterialUtils::ApplyMaterialToSelection(PreviewDetails[AssetID]->PreviewInstance->GetPathName());
		}

		return;

	}

	if (!FBridgeDragDropHelper::Instance->SurfaceToActorMap.Contains(AssetID)) return;

	FViewport* ActiveViewport = GEditor->GetActiveViewport();
	FEditorViewportClient* EditorViewClient = (FEditorViewportClient*)ActiveViewport->GetClient();

	AStaticMeshActor* SMActor = Cast<AStaticMeshActor>(FBridgeDragDropHelper::Instance->SurfaceToActorMap[AssetID]);

	if (bIsNormal)
	{
		UMaterialInstanceConstant* MInstance = Cast<UMaterialInstanceConstant>(MInstanceData.GetAsset());
		if (SMActor != nullptr)
		{
			SMActor->GetStaticMeshComponent()->SetMaterial(0, MInstance);
		}
	}
	else
	{
		if (SMActor != nullptr)
		{
			SMActor->GetStaticMeshComponent()->SetMaterial(0, CastChecked<UMaterialInterface>(PreviewDetails[AssetID]->PreviewInstance));
		}
	}

	if (bIsNormal)
	{
		return;
	}

	if (PreviewDetails.Contains(AssetID))
	{
		PreviewDetails[AssetID]->ActorsInLevel.Add(SMActor);
		//FBridgeDragDropHelper::Instance->OnAddProgressiveStageDataDelegate.ExecuteIfBound(PreviewerMeshData, AssetID, SMActor);
	}
}

void FImportProgressiveSurfaces::HandleHighInstanceLoad(FAssetData HighInstanceData, FString AssetID, FUAssetMeta AssetMetaData)
{
	AssetUtils::ConvertToVT(AssetMetaData);


	if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
	{
		AssetUtils::DeleteAsset(AssetMetaData.materialInstances[0].instancePath);
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}
	else if (AssetUtils::IsVTEnabled() && AssetMetaData.assetType != TEXT("3dplant"))
	{
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}

	if (!PreviewDetails.Contains(AssetID)) return;
	if (PreviewDetails[AssetID]->ActorsInLevel.Num() == 0)
	{
		PreviewDetails.Remove(AssetID);
		return;
	}

	for (AStaticMeshActor* UsedActor : PreviewDetails[AssetID]->ActorsInLevel)
	{
		if (!IsValid(UsedActor)) continue;
		if (!IsValid(UsedActor->GetStaticMeshComponent())) continue;
		if (!UsedActor) continue;
		if (UsedActor == nullptr) continue;			

		AssetUtils::ManageImportSettings(AssetMetaData);		

		//UMaterialInstanceConstant* HighInstance = Cast<UMaterialInstanceConstant>(HighInstanceData.GetAsset());		
		UsedActor->GetStaticMeshComponent()->SetMaterial(0, CastChecked<UMaterialInterface>(HighInstanceData.GetAsset()));
	}
	PreviewDetails.Remove(AssetID);
}


//Handle normal surfaces/decals/imperfections import through drag.
void FImportProgressiveSurfaces::HandleNormalMaterialLoad(FAssetData AssetInstanceData, FUAssetMeta AssetMetaData, float LocationOffset)
{	
	if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
	{
		AssetUtils::DeleteAsset(AssetMetaData.materialInstances[0].instancePath);
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
		FAssetData OverridenInstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(OverridenInstance));
		SpawnMaterialPreviewActor(AssetMetaData.assetID, LocationOffset, true, OverridenInstanceData);
		return;
	}

	// SpawnMaterialPreviewActor(AssetMetaData.assetID, LocationOffset, true, AssetInstanceData);
	// FBridgeDragDropHelper::Instance->OnNormalSurfaceDelegate.ExecuteIfBound(AssetInstanceData);

	if (FBridgeDragDropHelper::Instance->SurfaceToActorMap.Contains(AssetMetaData.assetID))
	{
		UMaterialInstanceConstant* MInstance = Cast<UMaterialInstanceConstant>(AssetInstanceData.GetAsset());
		AStaticMeshActor* Actor = Cast<AStaticMeshActor>(FBridgeDragDropHelper::Instance->SurfaceToActorMap[AssetMetaData.assetID]);
		if (Actor != nullptr)
		{
			Actor->GetStaticMeshComponent()->SetMaterial(0, MInstance);
		}
	}
}
