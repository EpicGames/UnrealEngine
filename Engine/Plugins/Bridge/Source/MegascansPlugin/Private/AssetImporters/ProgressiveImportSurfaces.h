// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "AssetRegistry/AssetData.h"

class AStaticMeshActor;
class FJsonObject;
class UMaterialInstanceConstant;
struct FUAssetMeta;

struct FProgressiveSurfaces
{	
	TArray<AStaticMeshActor*> ActorsInLevel;
	UMaterialInstanceConstant* PreviewInstance = nullptr;
	FString PreviewFolderPath;
	FString PreviewMeshPath;
};

class FImportProgressiveSurfaces
{

private:
	FImportProgressiveSurfaces() = default;
	static TSharedPtr<FImportProgressiveSurfaces> ImportProgressiveSurfacesInst;

	TMap<FString, TSharedPtr<FProgressiveSurfaces>> PreviewDetails;
	void SpawnMaterialPreviewActor(FString AssetID, float LocationOffset, bool bIsNormal = false, FAssetData MInstanceData = NULL);


public:
	static TSharedPtr<FImportProgressiveSurfaces> Get();
	void ImportAsset(TSharedPtr<FJsonObject> AssetImportJson, float LocationOffset, bool bIsNormal = false);

	void HandlePreviewInstanceLoad(FAssetData PreviewInstanceData, FString AssetID, float LocationOffset);
	void HandlePreviewTextureLoad(FAssetData TextureData, FString AssetID, FString Type);

	void HandleHighInstanceLoad(FAssetData HighInstanceData, FString AssetID, FUAssetMeta AssetMetaData);

	void HandleNormalMaterialLoad(FAssetData AssetInstanceData, FUAssetMeta AssetMetaData, float LocationOffset);
	
};
