// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetImporters/UAssetNormalImport.h"
#include "Dom/JsonObject.h"
#include "Utilities/MiscUtils.h"
#include "Misc/Paths.h"
#include "Utilities/MaterialUtils.h"




TSharedPtr<FImportUAssetNormal> FImportUAssetNormal::ImportUassetNormalInst;

TSharedPtr<FImportUAssetNormal> FImportUAssetNormal::Get()
{
	if (!ImportUassetNormalInst.IsValid())
	{
		ImportUassetNormalInst = MakeShareable(new FImportUAssetNormal);
	}
	return ImportUassetNormalInst;
}

void FImportUAssetNormal::ImportAsset(TSharedPtr<FJsonObject> AssetImportJson)
{

	FUAssetMeta AssetMetaData = AssetUtils::GetAssetMetaData(AssetImportJson->GetStringField(TEXT("importJson")));

	TArray<FString> AssetPaths;		
	
	FString AssetId = AssetImportJson->GetStringField(TEXT("assetId"));
	TArray<TSharedPtr<FJsonValue>> FilePathsArray = AssetImportJson->GetArrayField(TEXT("assetPaths"));

	for (TSharedPtr<FJsonValue> FilePath : FilePathsArray)
	{
		AssetPaths.Add(FilePath->AsString());
	}

	
	FString DestinationPath = AssetMetaData.assetRootPath;
	FString DestinationFolder = FPaths::Combine(FPaths::ProjectContentDir(), DestinationPath.Replace(TEXT("/Game/"), TEXT("")));

	/*FString AssetCopyMsg = TEXT("Importing : ") + AssetMetaData.assetName;
	FText AssetCopyMsgDialogMessage = FText::FromString(AssetCopyMsg);
	FScopedSlowTask AssetLoadprogress(1.0f, AssetCopyMsgDialogMessage, true);
	AssetLoadprogress.MakeDialog();*/
	if (AssetMetaData.assetType == TEXT("3dplant"))
	{
		CopyUassetFilesPlants(AssetPaths, DestinationFolder, AssetMetaData.assetTier);
	}
	else
	{
		CopyUassetFiles(AssetPaths, DestinationFolder);
	}

	if (AssetMetaData.assetType != TEXT("3dplant"))
	{
		AssetUtils::ConvertToVT(AssetMetaData);
	}

	if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
	{
		
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}
	else if (AssetUtils::IsVTEnabled() && AssetMetaData.assetType != TEXT("3dplant"))
	{
		UMaterialInstanceConstant* OverridenInstance = FMaterialUtils::CreateMaterialOverride(AssetMetaData);
		FMaterialUtils::ApplyMaterialInstance(AssetMetaData, OverridenInstance);
	}

	AssetUtils::SyncFolder(AssetMetaData.assetRootPath);

	FString AssetPath = TEXT("");
	if (AssetMetaData.assetType == TEXT("3dplant") || AssetMetaData.assetType == TEXT("3d"))
	{
		AssetPath = AssetMetaData.meshList[0].path;
	}
	else
	{ 
		AssetPath = AssetMetaData.materialInstances[0].instancePath;
	}
	if (AssetPath != TEXT(""))
	{
		AssetUtils::RegisterAsset(AssetPath);
	}

	AssetUtils::ManageImportSettings(AssetMetaData);
	AssetUtils::FocusOnSelected(DestinationPath);
	
	
}
