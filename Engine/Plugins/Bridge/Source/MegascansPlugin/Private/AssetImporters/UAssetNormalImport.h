// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Templates/SharedPointer.h"

class FJsonObject;




class FImportUAssetNormal 
	
{

private :
	FImportUAssetNormal() = default;
	static TSharedPtr<FImportUAssetNormal> ImportUassetNormalInst;

public:
	static TSharedPtr<FImportUAssetNormal> Get();
	void ImportAsset(TSharedPtr<FJsonObject> AssetImportJson);
};
