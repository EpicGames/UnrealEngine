// Copyright Epic Games, Inc. All Rights Reserved.
#include "AssetsImportController.h"
#include "Utilities/MiscUtils.h"
#include "AssetImporters/UAssetNormalImport.h"
#include "AssetImporters/ProgressiveImport3D.h"
#include "AssetImporters/ProgressiveImportSurfaces.h"
#include "ProjectUtilities/MetaHumanProjectUtilities.h"
#include "Misc/Paths.h"


#include "Dom/JsonObject.h"
#include "Import/MetaHumanImport.h"


TSharedPtr<FAssetsImportController> FAssetsImportController::AssetsImportController;


// Get instance of the class
TSharedPtr<FAssetsImportController> FAssetsImportController::Get()
{
	if (!AssetsImportController.IsValid())
	{
		AssetsImportController = MakeShareable(new FAssetsImportController);
	}
	return AssetsImportController;
}

void FAssetsImportController::DataReceived(const FString DataFromBridge)
{
	using namespace UE::MetaHuman;

	static const FString NormalExportMode = TEXT("normal");
	static const FString NormalDragExportMode = TEXT("normal_drag");
	static const FString ProgressiveExportMode = TEXT("progressive");

	float LocationOffset = 0.0f;
	TSharedPtr<FJsonObject> ImportDataObject = DeserializeJson(DataFromBridge);
	const TArray<TSharedPtr<FJsonValue>>& AssetsImportDataArray = ImportDataObject->GetArrayField(TEXT("exportPayload"));

	// If there are multiple MetaHumans in this import, then capture that
	TSet<FString> BatchImportCharacters;
	for (const TSharedPtr<FJsonValue>& AssetJson : AssetsImportDataArray)
	{
		EAssetImportType ImportType = JsonUtils::GetImportType(AssetJson->AsObject());
		if (ImportType == EAssetImportType::DHI_CHARACTER)
		{
			BatchImportCharacters.Add(AssetJson->AsObject()->GetStringField(TEXT("folderName")));
		}
	}

	for (const TSharedPtr<FJsonValue>& AssetJson : AssetsImportDataArray)
	{
		const EAssetImportType ImportType = JsonUtils::GetImportType(AssetJson->AsObject());
		const FString ExportMode = AssetJson->AsObject()->GetStringField(TEXT("exportMode"));

		if (ImportType == EAssetImportType::MEGASCANS_UASSET)
		{
			const FString AssetType = AssetJson->AsObject()->GetStringField(TEXT("assetType"));
			CopyMSPresets();
			//if (!SupportedAssetTypes.Contains(AssetType)) continue;

			if (ExportMode == NormalExportMode)
			{
				FImportUAssetNormal::Get()->ImportAsset(AssetJson->AsObject());
			}
			else if (ExportMode == ProgressiveExportMode || ExportMode == NormalDragExportMode)
			{
				bool bIsNormal = (ExportMode == TEXT("normal_drag")) ? true : false;

				if (AssetType == TEXT("3d") || AssetType == TEXT("3dplant"))
				{
					FImportProgressive3D::Get()->ImportAsset(AssetJson->AsObject(), LocationOffset, bIsNormal);
				}

				else if (AssetType == TEXT("surface") || AssetType == TEXT("atlas"))
				{
					FImportProgressiveSurfaces::Get()->ImportAsset(AssetJson->AsObject(), LocationOffset, bIsNormal);
				}

				if (AssetJson->AsObject()->GetIntegerField(TEXT("progressiveStage")) == 1 || bIsNormal)
				{
					LocationOffset += 200;
				}
			}
		}
		else if (ImportType == EAssetImportType::DHI_CHARACTER)
		{
			const TSharedPtr<FJsonObject>& AssetJsonObject = AssetJson->AsObject();
			if (ExportMode == ProgressiveExportMode)
			{
				// This MetaHuman is just being downloaded at the moment, do not import yet.
				return;
			}

			const FString CharacterName = AssetJsonObject->GetStringField(TEXT("folderName"));

			// Value is only present when set to true
			bool bIsBulkExported = BatchImportCharacters.Num() > 1;
			AssetJsonObject->TryGetBoolField(TEXT("isBulkExported"), bIsBulkExported);

			FMetaHumanImportDescription AssetImportDescription = {
				FPaths::Combine(AssetJsonObject->GetStringField(TEXT("characterPath")), CharacterName),
				AssetJsonObject->GetStringField(TEXT("commonPath")),
				CharacterName,
				AssetJsonObject->GetStringField(TEXT("id")),
				bIsBulkExported
			};

			const TArray<TSharedPtr<FJsonValue>>* AvailableMetahumans;
			if (ImportDataObject->TryGetArrayField(TEXT("availableMetahumans"), AvailableMetahumans))
			{
				for (const TSharedPtr<FJsonValue>& Entry : *AvailableMetahumans)
				{
					TSharedPtr<FJsonObject> EntryJsonObject = Entry->AsObject();

					// Value is only present when set to true
					bool bIsLegacy = false;
					EntryJsonObject->TryGetBoolField(TEXT("isLegacy"), bIsLegacy);

					AssetImportDescription.AccountMetaHumans.Emplace(FQuixelAccountMetaHumanEntry{
						EntryJsonObject->GetStringField(TEXT("name")),
						EntryJsonObject->GetStringField(TEXT("id")),
						bIsLegacy,
						EntryJsonObject->GetStringField(TEXT("version")),
					});
				}
			}

			FMetaHumanProjectUtilities::ImportMetaHuman(AssetImportDescription);
		}
	}
}
