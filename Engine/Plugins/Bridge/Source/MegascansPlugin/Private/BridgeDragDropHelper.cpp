// Copyright Epic Games, Inc. All Rights Reserved.
#include "BridgeDragDropHelper.h"

TSharedPtr<FBridgeDragDropHelperImpl> FBridgeDragDropHelper::Instance;

void FBridgeDragDropHelper::Initialize()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShareable(new FBridgeDragDropHelperImpl);
	}
}

void FBridgeDragDropHelperImpl::SetOnAddProgressiveStageData(FOnAddProgressiveStageDataCallbackInternal InDelegate)
{
    OnAddProgressiveStageDataDelegate = InDelegate;
}
