// Copyright Epic Games, Inc. All Rights Reserved.
#include "BridgeDragDropUtils.h"

TSharedPtr<FBridgeDragDropImpl> FBridgeDragDrop::Instance;

void FBridgeDragDrop::Initialize()
{
	if (!Instance.IsValid())
	{
		Instance = MakeShareable(new FBridgeDragDropImpl);
	}
}

void FBridgeDragDropImpl::SetOnAddProgressiveStageData(FOnAddProgressiveStageDataCallback InDelegate)
{
    OnAddProgressiveStageDataDelegate = InDelegate;
}
