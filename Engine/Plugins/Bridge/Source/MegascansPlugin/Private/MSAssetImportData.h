// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "MSAssetImportData.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(MSLiveLinkLog, Log, All);

//Types of imports this plugin will handle
enum EAssetImportType {
	MEGASCANS_SOURCE,
	MEGASCANS_UASSET,
	TEMPLATE,
	DHI_CHARACTER,
	NONE
};

// Data strcutures to hold json data for specific import types
//UAsset asset import data
struct FUAssetData
{
	
	FString ImportType; //Should be converted to EAssetImportType
	uint8 AssetTier;
	FString AssetType;
	FString ExportMode;
	int8 ProgressiveStage;
	FString ImportJsonPath;
	FString AssetId;
	TArray<FString> FilePaths;

};



//Asset metadata


USTRUCT()
struct FMaterialUsage {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString instanceID = "";
	UPROPERTY()
		FString materialSlot = "";
};

USTRUCT()
struct FMeshInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString meshID = "";
	UPROPERTY()
		FString name = "";
	UPROPERTY()
		FString path = "";
	UPROPERTY()
		int8 numberOfLods = 0;
	UPROPERTY()
		TArray<FMaterialUsage> materialUsage;

};

USTRUCT()
struct FFoliageTypeInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString path;
};

USTRUCT()
struct FMaterialParams {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString paramName;
	UPROPERTY()
		FString usedTextureID;
};


USTRUCT()
struct FMaterialInstanceInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString instanceID;
	UPROPERTY()
		FString instanceName;
	UPROPERTY()
		FString instancePath;
	UPROPERTY()
		FString instanceMaster;
	UPROPERTY()
		FString type;
	UPROPERTY()
		TArray<FMaterialParams> params;
};

USTRUCT()
struct FMasterMaterialInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString masterID;
	UPROPERTY()
		FString masterMaterialName;
	UPROPERTY()
		FString path;
};


USTRUCT()
struct FChannelPackedInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString channel = "";
	UPROPERTY()
		FString packedType = "";

};

USTRUCT()
struct FTextureUsage {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString matID = "";
	UPROPERTY()
		FString matParams = "";
};

USTRUCT()
struct FTexturesList {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString textureID = "";
	UPROPERTY()
		FString type = "";
	UPROPERTY()
		FString resolution = "";
	UPROPERTY()
		FString name = "";
	UPROPERTY()
		FString path = "";
	UPROPERTY()
		bool isChannelPacked = true;
	UPROPERTY()
		TArray<FChannelPackedInfo> channelPackInfo;
	UPROPERTY()
		TArray<FTextureUsage> pluggedIn;
};


USTRUCT()
struct FUAssetMeta
{
	GENERATED_BODY()
public:
	UPROPERTY()
		FString assetID;
	UPROPERTY()
		FString assetName;
	UPROPERTY()
		FString assetType;
	UPROPERTY()
		FString assetSubType;
	UPROPERTY()
		int8 assetTier = 0;
	UPROPERTY()
		FString assetRootPath;
	UPROPERTY()
		TArray<FMeshInfo> meshList;
	UPROPERTY()
		TArray<FString> foliageAssetPaths;
	UPROPERTY()
		TArray<FMaterialInstanceInfo> materialInstances;
	UPROPERTY()
		TArray<FTexturesList> textureSets;
	UPROPERTY()
		TArray<FMasterMaterialInfo> masterMaterials;

};
