// Copyright Epic Games, Inc. All Rights Reserved.
#include "IMegascansLiveLinkModule.h"
#include "TCPServer.h"
#include "BridgeDragDropHelper.h"
#include "UI/MaterialBlendingDetails.h"

#include "PropertyEditorModule.h"



#define LOCTEXT_NAMESPACE "MegascansPlugin"

class FQMSLiveLinkModule : public IMegascansLiveLinkModule
{
private : 
	FTCPServer* SocketListener = NULL;
public:
	virtual void StartupModule() override
	{
		if (SocketListener == NULL)
		{
			SocketListener = new FTCPServer();
		}

		FBridgeDragDropHelper::Initialize();

		auto& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(
			"MaterialBlendSettings",
			FOnGetDetailCustomizationInstance::CreateStatic(&BlendSettingsCustomization::MakeInstance)
		);
		PropertyModule.NotifyCustomizationModuleChanged();

	}

	virtual void ShutdownModule() override
	{
		if (SocketListener != nullptr)
		{
			SocketListener->Exit();
			delete SocketListener;
		}

		if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
		{
			auto& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout("MaterialBlendSettings");
		}
	}

};

IMPLEMENT_MODULE(FQMSLiveLinkModule, MegascansPlugin);

#undef LOCTEXT_NAMESPACE
