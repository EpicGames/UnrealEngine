// Copyright Epic Games, Inc. All Rights Reserved.
#include "TCPServer.h"
#include "HAL/RunnableThread.h"
#include "UObject/GarbageCollection.h"
#include "AssetsImportController.h"
#include "Sockets.h"
#include "TickableEditorObject.h"
#include "UObject/UObjectGlobals.h"

TQueue<FString> FTCPServer::ImportQueue;

/* Actually import data during the "Game tick" phase when components can be created and destroyed etc. */
class FDeferredPackageImporter : public FTickableEditorObject
{
public:
    FDeferredPackageImporter() {}
    virtual ~FDeferredPackageImporter() {}
    virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(DeferredPackageImporter, STATGROUP_Tickables); }
    virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }

    virtual void Tick(float DeltaTime) override
    {
        // Not sure if these are required
        if (IsGarbageCollecting() || UE::IsSavingPackage())
        {
            return;
        }

        if (FTCPServer::ImportQueue.IsEmpty())
        {
            return;
        }

        FString ImportData;
        FTCPServer::ImportQueue.Dequeue(ImportData);

        FAssetsImportController::Get()->DataReceived(ImportData);
    }
};

FTCPServer::FTCPServer()
{
    FThreadSafeCounter WorkerCounter;
    FString ThreadName(FString::Printf(TEXT("MegascansPlugin%i"), WorkerCounter.Increment()));
    ClientThread = FRunnableThread::Create(this, *ThreadName, 8 * 1024, TPri_Normal);
    static FDeferredPackageImporter PackageImporter;
}

FTCPServer::~FTCPServer()
{
    Stop();

    if (ClientThread != NULL)
    {
        ClientThread->Kill(true);
        delete ClientThread;
    }
}

bool FTCPServer::Init()
{
    Stopping = false;
    return true;
}

uint32 FTCPServer::Run()
{
    /*while (!Stopping)
    {
        FPlatformProcess::Sleep(0.3f);
    }*/

    return 0;
}

// bool FTCPServer::RecvMessage(FSocket* Socket, uint32 DataSize, FString& Message)
//{
//	check(Socket);
//	FArrayReaderPtr Datagram = MakeShareable(new FArrayReader(true));
//	int32 stuff = 16;
//	Datagram->Init(FMath::Min(DataSize, 65507u), 81920);
//	int32 BytesRead = 0;
//
//	if (Socket->Recv(Datagram->GetData(), Datagram->Num(), BytesRead))
//	{
//
//		char* Data = (char*)Datagram->GetData();
//		Data[BytesRead] = '\0';
//		FString message = UTF8_TO_TCHAR(Data);
//		Message = message;
//
//		return true;
//	}
//
//	return false;
// }

bool FTCPServer::HandleListenerConnectionAccepted(class FSocket *ClientSocket, const FIPv4Endpoint &ClientEndpoint)

{

    PendingClients.Enqueue(ClientSocket);
    return true;
}
