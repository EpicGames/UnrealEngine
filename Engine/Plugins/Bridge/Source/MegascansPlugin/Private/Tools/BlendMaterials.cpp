// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tools/BlendMaterials.h"
#include "AssetRegistry/AssetData.h"
#include "EditorAssetLibrary.h"
#include "Misc/Paths.h"
#include "Utilities/MiscUtils.h"
#include "Modules/ModuleManager.h"
#include "Utilities/MaterialUtils.h"
#include "MSSettings.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Engine/Texture.h"
#include "Misc/PackageName.h"
#include "MaterialEditingLibrary.h"

#include "Misc/MessageDialog.h"


#include "AssetRegistry/AssetRegistryModule.h"

#include "Engine/StreamableManager.h"
#include "Engine/AssetManager.h"
#include "Engine/Texture.h"

#include "Engine/AssetManager.h"
#include "AssetRegistry/AssetRegistryModule.h"

TSharedPtr<FMaterialBlend> FMaterialBlend::MaterialBlendInst;

TSharedPtr<FMaterialBlend> FMaterialBlend::Get()
{
    if (!MaterialBlendInst.IsValid())
    {

        MaterialBlendInst = MakeShareable(new FMaterialBlend);
    }
    return MaterialBlendInst;
}

void FMaterialBlend::BlendSelectedMaterials()

{

    const UMaterialBlendSettings *BlendSettings = GetDefault<UMaterialBlendSettings>();

    TArray<UMaterialInstanceConstant*> SelectedMaterialInstances = AssetUtils::GetSelectedAssets(FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("MaterialInstanceConstant")));

    if (SelectedMaterialInstances.Num() < 2)
    {
        FMessageDialog::Open(EAppMsgType::Ok, FText(FText::FromString("Select two or more material instances to perform this operation.")));
        return;
    }
    if (SelectedMaterialInstances.Num() > 3)
    {
        FMessageDialog::Open(EAppMsgType::Ok, FText(FText::FromString("Current Material Blending setup doesn't support more than 3 material instances.")));
        return;
        // Not supported number
    }

    if (MasterMaterialPath == TEXT(""))
    {
        FMessageDialog::Open(EAppMsgType::Ok, FText(FText::FromString("Master Material for blending was not found. Cancelling operation.")));
        return;
        // Master blend material not found
    }

    FString BlendInstanceName;
    FString BlendDestinationPath;

    if (BlendSettings->BlendedMaterialName != TEXT(""))
    {
        BlendInstanceName = BlendSettings->BlendedMaterialName;
    }

    if (BlendSettings->BlendedMaterialPath.Path != "")
    {
        BlendDestinationPath = BlendSettings->BlendedMaterialPath.Path;
    }

    if (!FPackageName::IsValidObjectPath(FPaths::Combine(BlendDestinationPath, BlendInstanceName)))
    {
        BlendInstanceName = DBlendInstanceName;
        BlendDestinationPath = DBlendDestinationPath;
    }

    // BlendInstanceName = GetUniqueAssetName(BlendDestinationPath, BlendInstanceName, true);
    if (UEditorAssetLibrary::DoesAssetExist(FPaths::Combine(BlendDestinationPath, BlendInstanceName)))
    {
        FMessageDialog::Open(EAppMsgType::Ok, FText(FText::FromString("A Blend material with same name already exists. Please choose a different name.")));
        return;
    }

    UMaterialInstanceConstant *InstancedBlendMaterial = FMaterialUtils::CreateInstanceMaterial(MasterMaterialPath, BlendDestinationPath, BlendInstanceName);

    if (InstancedBlendMaterial == nullptr)
    {
        FMessageDialog::Open(EAppMsgType::Ok, FText(FText::FromString("An error ocurred while creating Blend material.")));
        return;
    }

    auto BlendSetIt = BlendSets.CreateConstIterator();
    for (UMaterialInstanceConstant *SelectedInstance : SelectedMaterialInstances)
    {
        for (FString MapType : SupportedMapTypes)
        {
            UTexture *PluggedMap = UMaterialEditingLibrary::GetMaterialInstanceTextureParameterValue(SelectedInstance, FName(*MapType));
            if (PluggedMap)
            {
                FName TargetParameterName = *(*BlendSetIt + TEXT(" Layer ") + MapType + TEXT(" Map"));
                UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(InstancedBlendMaterial, TargetParameterName, PluggedMap);
            }
        }
        ++BlendSetIt;
    }
}

bool FMaterialBlend::ValidateSelectedAssets(TArray<FString> SelectedMaterials, FString &Failure)
{
    for (FString MaterialPath : SelectedMaterials)
    {
    }
    return false;
}

void FMaterialBlend::HandleTextureLoading(FAssetData TextureData)
{
    UTexture *TextureAsset = Cast<UTexture>(TextureData.GetAsset());
	check( TextureAsset != nullptr );
	if ( ! TextureAsset->VirtualTextureStreaming )
	{
		TextureAsset->PreEditChange(nullptr);
	    TextureAsset->VirtualTextureStreaming = 1;
	}
    TextureAsset->SetFlags(RF_Standalone);
    TextureAsset->PostEditChange();
}

void FMaterialBlend::ConvertToVirtualTextures(FUAssetMeta AssetMetaData)
{

    IAssetRegistry &AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

    FStreamableManager &Streamable = UAssetManager::GetStreamableManager();

    for (FTexturesList TextureMeta : AssetMetaData.textureSets)
    {
        FAssetData TextureData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(TextureMeta.path));

        if (!TextureData.IsValid())
            return;

        FSoftObjectPath ItemToStream = TextureData.ToSoftObjectPath();
        Streamable.RequestAsyncLoad(ItemToStream, FStreamableDelegate::CreateRaw(this, &FMaterialBlend::HandleTextureLoading, TextureData));
    }
}
