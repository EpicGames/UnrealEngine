// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

struct FAssetData;
struct FUAssetMeta;

class FMaterialBlend
{
private:
    FMaterialBlend() = default;
    static TSharedPtr<FMaterialBlend> MaterialBlendInst;
    FString DBlendDestinationPath = TEXT("/Game/BlendMaterials");
    FString DBlendInstanceName = TEXT("BlendMaterial");
    FString DBlendMasterMaterial = TEXT("SurfaceBlend_MasterMaterial");

    // TArray<FString> BlendSets = { "Base", "R", "G", "B", "A" };
    TArray<FString> BlendSets = {"Base", "Middle", "Top"};
    // TArray<FString> SupportedMapTypes = { "albedo", "normal", "roughness", "specular", "gloss", "displacement", "opacity", "translucency" };
    TArray<FString> SupportedMapTypes = {"albedo", "normal", "ARD"};

    bool ValidateSelectedAssets(TArray<FString> SelectedMaterials, FString &Failure);
    FString MasterMaterialPath = TEXT("/Game/MSPresets/M_MS_SurfaceBlend_Material/M_MS_SurfaceBlend_Material.M_MS_SurfaceBlend_Material");

    void HandleTextureLoading(FAssetData TextureData);

	

public:
    static TSharedPtr<FMaterialBlend> Get();
    void BlendSelectedMaterials();

    void ConvertToVirtualTextures(FUAssetMeta AssetMetaData);
};
