// Copyright Epic Games, Inc. All Rights Reserved.
#include "MSSettings.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MSSettings)



UMegascansSettings::UMegascansSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) , bCreateFoliage(true), bApplyToSelection(false)

{
	
}


UMaterialBlendSettings::UMaterialBlendSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), BlendedMaterialName(TEXT("BlendMaterial"))

{
	BlendedMaterialPath.Path = TEXT("/Game/BlendMaterials");
}

UMaterialAssetSettings::UMaterialAssetSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

UMaterialPresetsSettings::UMaterialPresetsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	
}

#if WITH_EDITOR
void UMaterialPresetsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	// TODO : Only save the property thats getting changed.
	// Use ToSoftObjectPath() rather than operator-> so we don't dereference an unset
	// or unloaded TSoftObjectPtr. An empty path persists when a slot is cleared,
	// which preserves the "Default material is used if field is left empty" contract.
	UMaterialAssetSettings* MatOverridePathSettings = GetMutableDefault<UMaterialAssetSettings>();
	MatOverridePathSettings->MasterMaterial3d = MasterMaterial3d.ToSoftObjectPath().ToString();
	MatOverridePathSettings->MasterMaterialPlant = MasterMaterialPlant.ToSoftObjectPath().ToString();
	MatOverridePathSettings->MasterMaterialSurface = MasterMaterialSurface.ToSoftObjectPath().ToString();

	MatOverridePathSettings->SaveConfig();

	

}

#endif


