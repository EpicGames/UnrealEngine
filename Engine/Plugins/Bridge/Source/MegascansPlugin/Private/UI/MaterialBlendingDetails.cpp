// Copyright Epic Games, Inc. All Rights Reserved.
#include "MaterialBlendingDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"


#include "Tools/BlendMaterials.h"

#define LOCTEXT_NAMESPACE "MegascansLiveLinkModule"

void BlendSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& BlendCategory = DetailBuilder.EditCategory("MaterialBlendSettings");

	FDetailWidgetRow BlendButton = BlendCategory.AddCustomRow(NSLOCTEXT("MaterialBlendSettings", "CreateMaterialBlend", "Create Material Blend"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(0, 5, 10, 5)
		[
			SNew(SButton)
			.ContentPadding(3)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.IsEnabled(this, &BlendSettingsCustomization::CanCreateMaterialBlend)
		.OnClicked(this, &BlendSettingsCustomization::CreateMaterialBlend)
		.Text(NSLOCTEXT("MaterialBlendSettings", "CreateMaterialBlend", "Create Material Blend"))
		]
		];

	//BlendButton.NameContent().VAlign(EVerticalAlignment::VAlign_Center);

}

TSharedRef<IDetailCustomization> BlendSettingsCustomization::MakeInstance()
{
	return MakeShareable(new BlendSettingsCustomization);
}

bool BlendSettingsCustomization::CanCreateMaterialBlend() const
{
	return true;
}

FReply BlendSettingsCustomization::CreateMaterialBlend()
{
	FMaterialBlend::Get()->BlendSelectedMaterials();
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE

