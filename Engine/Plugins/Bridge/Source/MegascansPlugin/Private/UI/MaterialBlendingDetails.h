// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "IDetailCustomization.h"

class FReply;

class BlendSettingsCustomization : public IDetailCustomization

{
public:
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	bool CanCreateMaterialBlend() const;
	FReply CreateMaterialBlend();

};
