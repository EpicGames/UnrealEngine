// Copyright Epic Games, Inc. All Rights Reserved.
#include "SMSWindow.h"

#include "DetailsViewArgs.h"
#include "PropertyEditorModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailsView.h"
#include "MSSettings.h"
#include "Widgets/Layout/SScrollBox.h"
#include "EditorAssetLibrary.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"


#define LOCTEXT_NAMESPACE "MegascansSettings"

TWeakPtr<SWindow> MSSettingsWindow;


class SMegascansSettings : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SMegascansSettings) {}
		SLATE_ARGUMENT(UMegascansSettings*, InMegascansSettings)
		SLATE_ARGUMENT(UMaterialBlendSettings*, InMaterialBlendSettings)
		SLATE_ARGUMENT(UMaterialPresetsSettings*, InMaterialOverrideSettings)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		//const UMegascansSettings* MegascansSettings = GetDefault<UMegascansSettings>();
		

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs MiscOptionsDetailsArgs;
		MiscOptionsDetailsArgs.bUpdatesFromSelection = false;
		MiscOptionsDetailsArgs.bLockable = false;
		MiscOptionsDetailsArgs.bAllowSearch = false;
		MiscOptionsDetailsArgs.bShowOptions = false;
		MiscOptionsDetailsArgs.bAllowFavoriteSystem = false;
		MiscOptionsDetailsArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		MiscOptionsDetailsArgs.ViewIdentifier = "MegascansSettings";

		MiscOptionsDetails = PropertyEditor.CreateDetailView(MiscOptionsDetailsArgs);

		FDetailsViewArgs MatBlendDetailsArgs;
		MatBlendDetailsArgs.bUpdatesFromSelection = false;
		MatBlendDetailsArgs.bLockable = false;
		MatBlendDetailsArgs.bAllowSearch = false;
		MatBlendDetailsArgs.bShowOptions = false;
		MatBlendDetailsArgs.bAllowFavoriteSystem = false;
		MatBlendDetailsArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		MatBlendDetailsArgs.ViewIdentifier = "MaterialBlendingSettings";

		MatBlendDetails = PropertyEditor.CreateDetailView(MatBlendDetailsArgs);

		FDetailsViewArgs MaterialOverrideArgs;
		MaterialOverrideArgs.bUpdatesFromSelection = false;
		MaterialOverrideArgs.bLockable = false;
		MaterialOverrideArgs.bAllowSearch = false;
		MaterialOverrideArgs.bShowOptions = false;
		MaterialOverrideArgs.bAllowFavoriteSystem = false;
		MaterialOverrideArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		MaterialOverrideArgs.ViewIdentifier = "MasterMaterialOverrides";

		MatOverrideDetails = PropertyEditor.CreateDetailView(MaterialOverrideArgs);

		

		ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(4, 4, 4, 4)
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					MiscOptionsDetails.ToSharedRef()
				]

				+SScrollBox::Slot()
					[
						MatOverrideDetails.ToSharedRef()
					]

				+ SScrollBox::Slot()
					[
						MatBlendDetails.ToSharedRef()
						
					]

			]

		];		

		if (InArgs._InMegascansSettings)
		{
			SetMegascansSettings(InArgs._InMegascansSettings);
		}

		if (InArgs._InMaterialBlendSettings)
		{
			SetBlendSettings(InArgs._InMaterialBlendSettings);
		}

		if (InArgs._InMaterialOverrideSettings)
		{
			SetMaterialOverrideSettings(InArgs._InMaterialOverrideSettings);
		}
		
	}



	void SetMegascansSettings(UMegascansSettings* InMegascansSettings)
	{
		MegascansSettings = InMegascansSettings;

		MiscOptionsDetails->SetObject(MegascansSettings);
	}

	void SetBlendSettings(UMaterialBlendSettings* InBlendSettings)
	{
		MaterialBlendSettings = InBlendSettings;
		MatBlendDetails->SetObject(MaterialBlendSettings);

	}

	void SetMaterialOverrideSettings(UMaterialPresetsSettings* InMaterialOverrideSettings)
	{
		MaterialOverrideSettings = InMaterialOverrideSettings;
		MatOverrideDetails->SetObject(MaterialOverrideSettings);

	}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(MegascansSettings);
		Collector.AddReferencedObject(MaterialBlendSettings);
		Collector.AddReferencedObject(MaterialOverrideSettings);
	}

private:

	TSharedPtr<IDetailsView> MiscOptionsDetails;
	TSharedPtr<IDetailsView> MatBlendDetails;
	TSharedPtr<IDetailsView> MatOverrideDetails;

	TObjectPtr<UMegascansSettings> MegascansSettings;
	TObjectPtr<UMaterialBlendSettings> MaterialBlendSettings;
	TObjectPtr<UMaterialPresetsSettings> MaterialOverrideSettings;

	virtual FString GetReferencerName() const override { return "MegascanSettings"; }
};
	




void MegascansSettingsWindow::OpenSettingsWindow()
{
	TSharedPtr<SWindow> ExistingWindow = MSSettingsWindow.Pin();
	if (ExistingWindow.IsValid())
	{
		ExistingWindow->BringToFront();
	}
	else
	{
		ExistingWindow = SNew(SWindow)
			.Title( LOCTEXT("MegascansSettings", "Megascans") )
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			
			.ClientSize(FVector2D(600, 600));

		/*TSharedPtr<SDockTab> OwnerTab = TabManager->GetOwnerTab();
		TSharedPtr<SWindow> RootWindow = OwnerTab.IsValid() ? OwnerTab->GetParentWindow() : TSharedPtr<SWindow>();
		if(RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
		}
		else
		{*/
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		//}
	}

	UMegascansSettings* MegascansSettings = GetMutableDefault<UMegascansSettings>();
	UMaterialBlendSettings* MaterialBlendSettings = GetMutableDefault<UMaterialBlendSettings>();
	UMaterialPresetsSettings* MaterialOverrideSettings = GetMutableDefault<UMaterialPresetsSettings>();

	//Settings override changing
	const UMaterialAssetSettings* MatAssetPathsSettings = GetDefault<UMaterialAssetSettings>();
	


	if (MatAssetPathsSettings->MasterMaterial3d != "None" && MatAssetPathsSettings->MasterMaterial3d != "")
	{
		if (UEditorAssetLibrary::DoesAssetExist(MatAssetPathsSettings->MasterMaterial3d))
		{
			MaterialOverrideSettings->MasterMaterial3d = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MatAssetPathsSettings->MasterMaterial3d));
		}
	}
	/*else {
		CopyMSPresets();
		FString MasterMaterialPath = TEXT("/Game/MSPresets/M_MS_Default_Material/M_MS_Default_Material.M_MS_Default_Material");
		if (UEditorAssetLibrary::DoesAssetExist(MasterMaterialPath))
		{
			MaterialOverrideSettings->MasterMaterial3d = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MasterMaterialPath));
		}

	}*/
	if (MatAssetPathsSettings->MasterMaterialPlant != "None" && MatAssetPathsSettings->MasterMaterialPlant != "")
	{
		if (UEditorAssetLibrary::DoesAssetExist(MatAssetPathsSettings->MasterMaterialPlant))
		{
			MaterialOverrideSettings->MasterMaterialPlant = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MatAssetPathsSettings->MasterMaterialPlant));
		}
	}
	/*else
	{
		CopyMSPresets();
		FString MasterMaterialPath = TEXT("/Game/MSPresets/M_MS_Foliage_Material/M_MS_Foliage_Material.M_MS_Foliage_Material");
		if (UEditorAssetLibrary::DoesAssetExist(MasterMaterialPath))
		{
			MaterialOverrideSettings->MasterMaterialPlant = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MasterMaterialPath));
		}
	}*/
	if (MatAssetPathsSettings->MasterMaterialSurface != "None" && MatAssetPathsSettings->MasterMaterialSurface != "")
	{
		if (UEditorAssetLibrary::DoesAssetExist(MatAssetPathsSettings->MasterMaterialSurface))
		{
			MaterialOverrideSettings->MasterMaterialSurface = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MatAssetPathsSettings->MasterMaterialSurface));
		}
	}
	/*else
	{
		CopyMSPresets();
		FString MasterMaterialPath = TEXT("/Game/MSPresets/M_MS_Surface_Material/M_MS_Surface_Material.M_MS_Surface_Material");
		if (UEditorAssetLibrary::DoesAssetExist(MasterMaterialPath))
		{
			MaterialOverrideSettings->MasterMaterialSurface = CastChecked<UMaterial>(UEditorAssetLibrary::LoadAsset(MasterMaterialPath));
		}

	}*/
	ExistingWindow->SetContent(
		SNew(SMegascansSettings)
		.InMegascansSettings(MegascansSettings)
		.InMaterialBlendSettings(MaterialBlendSettings)
		.InMaterialOverrideSettings(MaterialOverrideSettings)

	);
	
	
	//ExistingWindow->SetOnWindowClosed(FOnWindowClosed::CreateStatic(&MegascansSettingsWindow::SaveSettings, MegascansSettings));	
	
	MSSettingsWindow = ExistingWindow;
	
}

void MegascansSettingsWindow::SaveSettings(const TSharedRef<SWindow>& Window, UMegascansSettings* MegascansSettings)
{	
	MegascansSettings->SaveConfig();

	//Save Master material overrides.
	
	const UMaterialPresetsSettings* MatPresetsSettings = GetDefault<UMaterialPresetsSettings>();
	UMaterialAssetSettings* MatOverridePathSettings = GetMutableDefault<UMaterialAssetSettings>();

	MatOverridePathSettings->MasterMaterial3d = MatPresetsSettings->MasterMaterial3d->GetPathName();
	MatOverridePathSettings->MasterMaterialPlant = MatPresetsSettings->MasterMaterialPlant->GetPathName();
	MatOverridePathSettings->MasterMaterialSurface = MatPresetsSettings->MasterMaterialSurface->GetPathName();
	MatOverridePathSettings->SaveConfig();
	

}





#undef LOCTEXT_NAMESPACE
