// Copyright Epic Games, Inc. All Rights Reserved.
#include "Utilities/MaterialUtils.h"
#include "Engine/StaticMeshActor.h"
#include "Utilities/MiscUtils.h"
#include "MSSettings.h"

#include "Factories/MaterialInstanceConstantFactoryNew.h"

#include "Engine/Texture.h"
#include "MaterialEditingLibrary.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "EditorAssetLibrary.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMesh.h"

#include "Engine/Level.h"
#include "Components/StaticMeshComponent.h"

#include "Editor.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Misc/Paths.h"
#include "Selection.h"

UMaterialInstanceConstant *FMaterialUtils::CreateInstanceMaterial(const FString &MasterMaterialPath, const FString &InstanceDestination, const FString &AssetName)
{
    UMaterialInterface *MasterMaterial = Cast<UMaterialInterface>(UEditorAssetLibrary::LoadAsset(MasterMaterialPath));
    if (MasterMaterial == nullptr)
        return nullptr;

    IAssetTools &AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
    UMaterialInstanceConstantFactoryNew *Factory = NewObject<UMaterialInstanceConstantFactoryNew>();
    UMaterialInstanceConstant *MaterialInstance = CastChecked<UMaterialInstanceConstant>(AssetTools.CreateAsset(AssetName, InstanceDestination, UMaterialInstanceConstant::StaticClass(), Factory));
    UMaterialEditingLibrary::SetMaterialInstanceParent(MaterialInstance, MasterMaterial);
    if (MaterialInstance)
    {
        MaterialInstance->SetFlags(RF_Standalone);
        MaterialInstance->MarkPackageDirty();
        MaterialInstance->PostEditChange();
    }
    return MaterialInstance;
}

UMaterialInstanceConstant *FMaterialUtils::CreateMaterialOverride(FUAssetMeta AssetMetaData)
{
    FString MasterMaterialPath;

    FString InstanceName = AssetMetaData.materialInstances[0].instanceName;

    // Create instance from the new master
    const UMaterialAssetSettings *MatAssetPathsSettings = GetDefault<UMaterialAssetSettings>();

    if (FMaterialUtils::ShouldOverrideMaterial(AssetMetaData.assetType))
    {

        if (AssetMetaData.assetType == TEXT("3d"))
        {
            MasterMaterialPath = MatAssetPathsSettings->MasterMaterial3d;
        }
        else if (AssetMetaData.assetType == TEXT("3dplant"))
        {
            MasterMaterialPath = MatAssetPathsSettings->MasterMaterialPlant;
        }

        else if (AssetMetaData.assetType == TEXT("surface"))
        {
            MasterMaterialPath = MatAssetPathsSettings->MasterMaterialSurface;
        }
    }

    // Get name of VT master material
    else
    {
        MasterMaterialPath = AssetMetaData.masterMaterials[0].path;

        FString VTMaterialDirectory = FPaths::GetPath(MasterMaterialPath) + TEXT("_VT");
        FString MasterMaterialName = FPaths::GetBaseFilename(MasterMaterialPath) + TEXT("_VT");
        MasterMaterialName += TEXT(".") + MasterMaterialName;

        MasterMaterialPath = FPaths::Combine(VTMaterialDirectory, MasterMaterialName);
    }

    AssetUtils::DeleteAsset(AssetMetaData.materialInstances[0].instancePath);
    // Check if instance exists then suggest a unique name

    UMaterialInstanceConstant *OverrideInstance = CreateInstanceMaterial(MasterMaterialPath, AssetMetaData.assetRootPath, InstanceName);

    IAssetRegistry &AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

    for (FTexturesList Texture : AssetMetaData.textureSets)
    {

        if (UMaterialEditingLibrary::GetMaterialInstanceTextureParameterValue(OverrideInstance, FName(*Texture.type)))
        {

            FAssetData TextureData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(Texture.path));

            UTexture *TextureAsset = Cast<UTexture>(TextureData.GetAsset());
            if (UMaterialEditingLibrary::SetMaterialInstanceTextureParameterValue(OverrideInstance, FName(*Texture.type), TextureAsset))
            {
                OverrideInstance->SetFlags(RF_Standalone);
                OverrideInstance->MarkPackageDirty();
                OverrideInstance->PostEditChange();
            }
        }
    }
    // AssetUtils::SavePackage(OverrideInstance);

    return OverrideInstance;
}

void FMaterialUtils::ApplyMaterialInstance(FUAssetMeta AssetMetaData, UMaterialInstanceConstant *MaterialInstance)
{
    IAssetRegistry &AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

    for (FMeshInfo MeshMeta : AssetMetaData.meshList)
    {
        FAssetData MeshData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(MeshMeta.path));
        UStaticMesh *AssetMesh = Cast<UStaticMesh>(MeshData.GetAsset());
        AssetMesh->SetMaterial(0, CastChecked<UMaterialInterface>(MaterialInstance));

        AssetMesh->MarkPackageDirty();
        AssetMesh->PostEditChange();
        // AssetUtils::SavePackage(AssetMesh);
    }
}

bool FMaterialUtils::ShouldOverrideMaterial(const FString &AssetType)
{
    const UMegascansSettings *MegascansSettings = GetDefault<UMegascansSettings>();
    const UMaterialAssetSettings *MatAssetPathsSettings = GetDefault<UMaterialAssetSettings>();

    if (AssetType == TEXT("3d") && MatAssetPathsSettings->MasterMaterial3d != "None" && MatAssetPathsSettings->MasterMaterial3d != "")
    {
        return true;
    }
    else if (AssetType == TEXT("surface") && MatAssetPathsSettings->MasterMaterialSurface != "None" && MatAssetPathsSettings->MasterMaterialSurface != "")
    {
        return true;
    }

    else if (AssetType == TEXT("3dplant") && MatAssetPathsSettings->MasterMaterialPlant != "None" && MatAssetPathsSettings->MasterMaterialPlant != "")
    {
        return true;
    }

    return false;
}

TArray<AStaticMeshActor *> FMaterialUtils::ApplyMaterialToSelection(const FString &InstancePath)
{
    IAssetRegistry &AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
    FAssetData InstanceData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(InstancePath));
    UMaterialInstanceConstant *MaterialInstance = Cast<UMaterialInstanceConstant>(UEditorAssetLibrary::LoadAsset(InstancePath));

    USelection *SelectedActors = GEditor->GetSelectedActors();
    TArray<AActor *> Actors;
    TArray<AStaticMeshActor *> SelectedStaticMeshes;

    TArray<ULevel *> UniqueLevels;
    for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
    {
        AActor *Actor = Cast<AActor>(*Iter);
        TArray<UStaticMeshComponent *> Components;
        Actor->GetComponents(Components);

        if (Components.Num() > 0)
        {
            AStaticMeshActor *SelectedStaticMesh = Cast<AStaticMeshActor>(Actor);
            if (SelectedStaticMesh)
            {
                SelectedStaticMeshes.Add(SelectedStaticMesh);
            }
        }

        for (int32 i = 0; i < Components.Num(); i++)
        {
            UStaticMeshComponent *MeshComponent = Components[i];
            int32 mCnt = MeshComponent->GetNumMaterials();
            for (int j = 0; j < mCnt; j++)
            {

                MeshComponent->SetMaterial(j, MaterialInstance);
            }
        }
    }

    return SelectedStaticMeshes;
}
