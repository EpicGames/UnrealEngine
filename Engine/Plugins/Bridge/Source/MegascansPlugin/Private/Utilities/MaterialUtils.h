// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"

class AStaticMeshActor;
class UMaterialInstanceConstant;
struct FUAssetMeta;

class FMaterialUtils
{
public:
	static UMaterialInstanceConstant* CreateInstanceMaterial(const FString& MasterMaterialPath, const FString& InstanceDestination, const FString& AssetName);
	static bool ShouldOverrideMaterial(const FString& AssetType);
	static TArray<AStaticMeshActor*> ApplyMaterialToSelection(const FString& InstancePath );
	static UMaterialInstanceConstant* CreateMaterialOverride(FUAssetMeta AssetMetaData);
	static void ApplyMaterialInstance(FUAssetMeta AssetMetaData, UMaterialInstanceConstant* MaterialInstance);	

};
