// Copyright Epic Games, Inc. All Rights Reserved.
#include "Utilities/MiscUtils.h"
#include "Engine/StaticMeshActor.h"
#include "Utilities/MaterialUtils.h"

#include "MSSettings.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"




#include "Materials/MaterialInstanceConstant.h"
#include "Misc/FileHelper.h"
#include "FileHelpers.h"

#include "EditorAssetLibrary.h"
#include "JsonObjectConverter.h"
#include "InstancedFoliageActor.h"
#include "FoliageType_InstancedStaticMesh.h"

#include "Editor.h"



#include "Tools/BlendMaterials.h"

TSharedPtr<FJsonObject> DeserializeJson(const FString& JsonStringData)
{
	TSharedPtr<FJsonObject> JsonDataObject;
	bool bIsDeseriliazeSuccessful = false;
	TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonStringData);
	bIsDeseriliazeSuccessful = FJsonSerializer::Deserialize(JsonReader, JsonDataObject);
	return JsonDataObject;
}

FString GetPluginPath()
{
	FString PluginName = TEXT("Bridge");
	FString PluginsPath = FPaths::EnginePluginsDir();

	FString MSPluginPath = FPaths::Combine(PluginsPath, PluginName);
	return MSPluginPath;
}

FString GetSourceMSPresetsPath()
{
	FString MaterialPresetsPath = FPaths::Combine(GetPluginPath(), TEXT("Content"), GetMSPresetsName());
	return MaterialPresetsPath;
}

FString GetMSPresetsName()
{
	return TEXT("MSPresets");
}

bool CopyMaterialPreset(const FString& MaterialName)
{
	FString MaterialSourceFolderPath = FPaths::Combine(GetSourceMSPresetsPath(), MaterialName);
	MaterialSourceFolderPath = FPaths::ConvertRelativePathToFull(MaterialSourceFolderPath);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*MaterialSourceFolderPath))
	{
		return false;
	}

	FString MaterialDestinationPath = FPaths::ProjectContentDir();
	MaterialDestinationPath = FPaths::Combine(MaterialDestinationPath, GetMSPresetsName());
	MaterialDestinationPath = FPaths::Combine(MaterialDestinationPath, MaterialName);
	
	if (!PlatformFile.DirectoryExists(*MaterialDestinationPath))
	{
		if (!PlatformFile.CreateDirectoryTree(*MaterialDestinationPath))
		{
			return false;
		}

		if (!PlatformFile.CopyDirectoryTree(*MaterialDestinationPath, *MaterialSourceFolderPath, true))
		{
			return false;
		}
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	TArray<FString> MaterialBasePath;
	FString BasePath = FPaths::Combine(FPaths::ProjectContentDir(), GetMSPresetsName());
	MaterialBasePath.Add("/Game/MSPresets");
	MaterialBasePath.Add(FPaths::Combine(TEXT("/Game/MSPresets"), MaterialName));
	if (PlatformFile.DirectoryExists(*FPaths::Combine(TEXT("/Game/MSPresets"), MaterialName, TEXT("Functions"))))
	{
		MaterialBasePath.Add(FPaths::Combine(TEXT("/Game/MSPresets"), MaterialName, TEXT("Functions")));
	}
	AssetRegistryModule.Get().ScanPathsSynchronous(MaterialBasePath, true);
	return true;
}

bool CopyPresetTextures()
{
	FString TexturesDestinationPath = FPaths::ProjectContentDir();

	TexturesDestinationPath = FPaths::Combine(TexturesDestinationPath, GetMSPresetsName(), TEXT("MSTextures"));

	FString TexturesSourceFolderPath = FPaths::Combine(GetSourceMSPresetsPath(), TEXT("MSTextures"));
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*TexturesDestinationPath))
	{
		if (!PlatformFile.CreateDirectoryTree(*TexturesDestinationPath))
		{
			return false;
		}

		if (!PlatformFile.CopyDirectoryTree(*TexturesDestinationPath, *TexturesSourceFolderPath, true))
		{
			return false;
		}
	}

	return true;
}

void CopyMSPresets()
{
	FString MaterialsDestination = FPaths::ProjectContentDir();
	MaterialsDestination = FPaths::Combine(MaterialsDestination, GetMSPresetsName());
	FString MaterialsSourceFolderPath = GetSourceMSPresetsPath();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	PlatformFile.CreateDirectoryTree(*MaterialsDestination);

	TArray<FString> SourceFiles;
	PlatformFile.FindFilesRecursively(SourceFiles, *MaterialsSourceFolderPath, NULL);

	for (FString SourceFile : SourceFiles)
	{
		FString DestinationFile = FPaths::Combine(MaterialsDestination, SourceFile.Replace(*MaterialsSourceFolderPath, TEXT("")));

		FString BaseDestinationDirectory = FPaths::GetPath(DestinationFile);
		PlatformFile.CreateDirectoryTree(*BaseDestinationDirectory);

		PlatformFile.CopyFile(*DestinationFile, *SourceFile);
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray<FString> SyncPaths;
	SyncPaths.Add(TEXT("/Game/MSPresets"));
	AssetRegistryModule.Get().ScanPathsSynchronous(SyncPaths, true);
}

void AssetUtils::FocusOnSelected(const FString& Path)
{
	TArray<FString> Folders;
	Folders.Add(Path);
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	IContentBrowserSingleton& ContentBrowserSingleton = ContentBrowserModule.Get();
	ContentBrowserSingleton.SyncBrowserToFolders(Folders);
}

void AssetUtils::SavePackage(UObject* SourceObject)
{
	/*TArray<UObject*> InputObjects;
	InputObjects.Add(SourceObject);
	UPackageTools::SavePackagesForObjects(InputObjects);*/

	TArray<UPackage*> PackagesToSave;
	PackagesToSave.Add(SourceObject->GetPackage());
	UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);
}

void AssetUtils::DeleteDirectory(FString TargetDirectory)
{
	TArray<FString> PreviewAssets = UEditorAssetLibrary::ListAssets(TargetDirectory);

	for (FString AssetPath : PreviewAssets)
	{
		DeleteAsset(AssetPath);
	}

	UEditorAssetLibrary::DeleteDirectory(TargetDirectory);
}

bool AssetUtils::DeleteAsset(const FString& AssetPath)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(AssetPath));

	if (AssetData.IsAssetLoaded())
	{
		if (UEditorAssetLibrary::DeleteLoadedAsset(AssetData.GetAsset()))
		{
			return true;
		}
	}
	else
	{
		if (UEditorAssetLibrary::DeleteAsset(AssetPath))
		{
			return true;
		}
	}

	return false;
}

FUAssetMeta AssetUtils::GetAssetMetaData(const FString& JsonPath)
{
	FString UassetMetaString;
	FFileHelper::LoadFileToString(UassetMetaString, *JsonPath);
	FUAssetMeta AssetMetaData;
	FJsonObjectConverter::JsonObjectStringToUStruct(UassetMetaString, &AssetMetaData);

	return AssetMetaData;
}

TArray<UMaterialInstanceConstant*> AssetUtils::GetSelectedAssets(const FTopLevelAssetPath& AssetClass)
{
	check(!AssetClass.IsNull());

	TArray<UMaterialInstanceConstant*> ObjectArray;

	TArray<FAssetData> AssetDatas;
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	IContentBrowserSingleton& ContentBrowserSingleton = ContentBrowserModule.Get();
	ContentBrowserSingleton.GetSelectedAssets(AssetDatas);

	for (FAssetData SelectedAsset : AssetDatas)
	{
		if (SelectedAsset.AssetClassPath == AssetClass)
		{
			ObjectArray.Add(CastChecked<UMaterialInstanceConstant>(UEditorAssetLibrary::LoadAsset(SelectedAsset.GetObjectPathString())));
		}

		if (SelectedAsset.AssetClassPath == AssetClass)
		{
			ObjectArray.Add(CastChecked<UMaterialInstanceConstant>(UEditorAssetLibrary::LoadAsset(SelectedAsset.GetObjectPathString())));
		}
	}
	return ObjectArray;
}

void AssetUtils::AddFoliageTypesToLevel(TArray<FString> FoliageTypePaths)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	for (FString FoliageTypePath : FoliageTypePaths)
	{
		FAssetData AssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(FoliageTypePath));
		if (!AssetData.IsValid()) return;

		auto* CurrentWorld = GEditor->GetEditorWorldContext().World();
		AInstancedFoliageActor* IFA = AInstancedFoliageActor::GetInstancedFoliageActorForCurrentLevel(CurrentWorld, true);
		IFA->AddFoliageType(Cast<UFoliageType_InstancedStaticMesh>(AssetData.GetAsset()));
	}
}

void AssetUtils::ManageImportSettings(FUAssetMeta AssetMetaData)
{
	const UMegascansSettings* MegascansSettings = GetDefault<UMegascansSettings>();

	if ((AssetMetaData.assetType == TEXT("surface") || AssetMetaData.assetType == TEXT("atlas")) && MegascansSettings->bApplyToSelection)
	{
		FMaterialUtils::ApplyMaterialToSelection(AssetMetaData.materialInstances[0].instancePath);
	}

	if (AssetMetaData.assetType == TEXT("3dplant") && MegascansSettings->bCreateFoliage)
	{
		AssetUtils::AddFoliageTypesToLevel(AssetMetaData.foliageAssetPaths);
	}
}

// Get import type based on incoming json
EAssetImportType JsonUtils::GetImportType(TSharedPtr<FJsonObject> ImportJsonObject)
{
	FString ExportType;
	if (ImportJsonObject->TryGetStringField(TEXT("exportType"), ExportType))
	{
		return (ExportType == TEXT("megascans_uasset")) ? EAssetImportType::MEGASCANS_UASSET : (ExportType == TEXT("megascans_source")) ? EAssetImportType::MEGASCANS_SOURCE : (ExportType == TEXT("dhi")) ? EAssetImportType::DHI_CHARACTER : (ExportType == TEXT("template")) ? EAssetImportType::TEMPLATE : EAssetImportType::NONE;

	}
	else
	{
		return EAssetImportType::NONE;
	}
}

TSharedPtr<FUAssetData> JsonUtils::ParseUassetJson(TSharedPtr<FJsonObject> ImportJsonObject)
{
	TSharedPtr<FUAssetData> ImportData = MakeShareable(new FUAssetData);
	TArray<FString> AssetPaths;
	ImportData->AssetTier = ImportJsonObject->GetIntegerField(TEXT("assetTier"));
	ImportData->AssetType = ImportJsonObject->GetStringField(TEXT("assetType"));
	ImportData->ExportMode = ImportJsonObject->GetStringField(TEXT("exportMode"));
	ImportData->ImportJsonPath = ImportJsonObject->GetStringField(TEXT("importJson"));
	ImportData->ImportType = ImportJsonObject->GetStringField(TEXT("exportType"));
	ImportData->AssetId = ImportJsonObject->GetStringField(TEXT("assetId"));
	ImportData->ProgressiveStage = ImportJsonObject->GetIntegerField(TEXT("progressiveStage"));

	TArray<TSharedPtr<FJsonValue>> FilePathsArray = ImportJsonObject->GetArrayField(TEXT("assetPaths"));

	for (TSharedPtr<FJsonValue> FilePath : FilePathsArray)
	{
		AssetPaths.Add(FilePath->AsString());
	}

	ImportData->FilePaths = AssetPaths;

	return ImportData;
}

void CopyUassetFiles(TArray<FString> FilesToCopy, FString DestinationDirectory)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.CreateDirectoryTree(*DestinationDirectory);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	
	TArray<FString> NewFilePaths;
	NewFilePaths.Reserve(FilesToCopy.Num());
	
	for (const FString& FileToCopy : FilesToCopy)
	{
		const FString DestinationFile = FPaths::Combine(DestinationDirectory, FPaths::GetCleanFilename(FileToCopy));
		PlatformFile.CopyFile(*DestinationFile, *FileToCopy);
		NewFilePaths.Add(DestinationFile);
	}

	AssetRegistryModule.Get().ScanFilesSynchronous(NewFilePaths, true);
}

void AssetUtils::SyncFolder(const FString& TargetFolder)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TArray<FString> SyncPaths;
	SyncPaths.Add(TargetFolder);
	AssetRegistry.ScanPathsSynchronous(SyncPaths, true);

	// Testing syncing
}

void AssetUtils::RegisterAsset(const FString& PackagePath)
{
	IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	FAssetData CharacterAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath(PackagePath));
	UObject* CharacterObject = CharacterAssetData.GetAsset();
}

void AssetUtils::ConvertToVT(FUAssetMeta AssetMetaData)
{
	if (IsVTEnabled())
	{

		FMaterialBlend::Get()->ConvertToVirtualTextures(AssetMetaData);

		// Call Master material override maybe to avoid all other steps
		// Create a new instance of the VT enabled master material based on which original material was used
		// Plug all the textures to this new material instance
		// Apply the newly generated material instance to the mesh incase of 3d assets
	}
}

bool AssetUtils::IsVTEnabled()
{
	static const auto CVarVirtualTexturesEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VirtualTextures"));
	check(CVarVirtualTexturesEnabled);
	static const auto CVarVirtualTexturesImportEnabled = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.VT.EnableAutoImport"));
	check(CVarVirtualTexturesEnabled);

	const bool bVirtualTextureEnabled = CVarVirtualTexturesEnabled->GetValueOnAnyThread() != 0;
	const bool bVirtualTextureImportEnabled = CVarVirtualTexturesImportEnabled->GetValueOnAnyThread() != 0;

	if (bVirtualTextureEnabled && bVirtualTextureImportEnabled)
	{
		return true;
	}

	return false;
}

void CopyUassetFilesPlants(TArray<FString> FilesToCopy, FString DestinationDirectory, const int8& AssetTier)
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.CreateDirectoryTree(*DestinationDirectory);
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	const FString FoliageDestination = FPaths::Combine(DestinationDirectory, TEXT("Foliage"));

	if (AssetTier < 4)
	{
		PlatformFile.CreateDirectoryTree(*FoliageDestination);
	}

	TArray<FString> NewFilePaths;
	NewFilePaths.Reserve(FilesToCopy.Num());
	
	for (const FString& FileToCopy : FilesToCopy)
	{
		FString DestinationFile = TEXT("");
		const FString TypeConvention = FPaths::GetBaseFilename(FileToCopy).Left(2);
		if (TypeConvention == TEXT("FT"))
		{
			DestinationFile = FPaths::Combine(FoliageDestination, FPaths::GetCleanFilename(FileToCopy));
		}
		else
		{
			DestinationFile = FPaths::Combine(DestinationDirectory, FPaths::GetCleanFilename(FileToCopy));
		}
		PlatformFile.CopyFile(*DestinationFile, *FileToCopy);
		NewFilePaths.Add(DestinationFile);
	}

	AssetRegistryModule.Get().ScanFilesSynchronous(NewFilePaths, true);
}
