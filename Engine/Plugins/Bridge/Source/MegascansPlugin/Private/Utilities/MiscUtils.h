// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include  "MSAssetImportData.h"


#include "MSAssetImportData.h"

class FJsonObject;
class UMaterialInstanceConstant;



TSharedPtr<FJsonObject>  DeserializeJson(const FString& JsonStringData);

FString GetPluginPath();
FString GetSourceMSPresetsPath();

bool CopyMaterialPreset(const FString & MaterialName);
FString GetMSPresetsName();
void CopyMSPresets();
bool CopyPresetTextures();
void CopyUassetFiles(TArray<FString> FilesToCopy, FString DestinationDirectory);
void CopyUassetFilesPlants(TArray<FString> FilesToCopy, FString DestinationDirectory, const int8 & AssetTier);


namespace AssetUtils {	//template<typename T>

	void FocusOnSelected(const FString& Path);
	void SavePackage(UObject* SourceObject);
	void DeleteDirectory(FString TargetDirectory);
	bool DeleteAsset(const FString& AssetPath);
	FUAssetMeta GetAssetMetaData(const FString& JsonPath);
	TArray<UMaterialInstanceConstant*> GetSelectedAssets(const FTopLevelAssetPath& AssetClass);
	void AddFoliageTypesToLevel(TArray<FString> FoliageTypePaths);

	//To manage import settings like Auto-Populate Foliage types etc.
	void ManageImportSettings(FUAssetMeta AssetMetaData);
	void SyncFolder(const FString& TargetFolder);
	void RegisterAsset(const FString& PackagePath);

	void ConvertToVT(FUAssetMeta AssetMetaData);
	bool IsVTEnabled();

}

namespace JsonUtils
{
	EAssetImportType GetImportType(TSharedPtr<FJsonObject> ImportJsonObject);
	void ParseImportJson(const FString& InputImportData);
	TSharedPtr<FUAssetData> ParseUassetJson(TSharedPtr<FJsonObject> ImportJsonObject);
}

