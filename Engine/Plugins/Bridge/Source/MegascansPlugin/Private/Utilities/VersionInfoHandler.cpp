// Copyright Epic Games, Inc. All Rights Reserved.
#include "VersionInfoHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VersionInfoHandler)

UVersionInfoHandler* UVersionInfoHandler::Get()
{
	return GetMutableDefault <UVersionInfoHandler>();
};
