// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VersionInfoHandler.generated.h"

USTRUCT()
struct FAssetInfo {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString path = "";
	UPROPERTY()
		FString version = "";
};

USTRUCT()
struct FVersionData
{
	GENERATED_BODY()
public:
	UPROPERTY()
		TArray<FAssetInfo> assets;
};


USTRUCT()
struct FAssetImportTime {
	GENERATED_BODY()
public:
	UPROPERTY()
		FString path = "";
	UPROPERTY()
		FString time = "";
};

USTRUCT()
struct FImportTimeData
{
	GENERATED_BODY()
public:
	UPROPERTY()
		TArray<FAssetImportTime> assets;
};

UCLASS(Blueprintable)
class UVersionInfoHandler : public UObject
{
	GENERATED_BODY()
private:

public:
	UFUNCTION(BlueprintCallable, Category = Python)
		static UVersionInfoHandler* Get();

	UPROPERTY()
		FVersionData CommonVersionData;
};
