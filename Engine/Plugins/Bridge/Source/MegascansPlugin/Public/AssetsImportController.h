// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/SharedPointer.h"

#define UE_API MEGASCANSPLUGIN_API

class FAssetsImportController
{
private:
	FAssetsImportController() = default;
	static UE_API TSharedPtr<FAssetsImportController> AssetsImportController;
	TArray<FString> SupportedAssetTypes = {
		TEXT("3d"),
		TEXT("3dplant"),
		TEXT("atlas"),
		TEXT("surface")
	};

public:	
	static UE_API TSharedPtr<FAssetsImportController> Get();
	UE_API void DataReceived(const FString DataFromBridge);
};

#undef UE_API
