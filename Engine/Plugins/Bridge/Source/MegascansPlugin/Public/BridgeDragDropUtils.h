// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"

#define UE_API MEGASCANSPLUGIN_API

struct FAssetData;

class AStaticMeshActor;

DECLARE_DELEGATE_ThreeParams(FOnAddProgressiveStageDataCallback, FAssetData AssetData, FString AssetId, AStaticMeshActor* SpawnedActor);

class FBridgeDragDropImpl : public TSharedFromThis<FBridgeDragDropImpl>
{
public:
    FOnAddProgressiveStageDataCallback OnAddProgressiveStageDataDelegate;

    UE_API void SetOnAddProgressiveStageData(FOnAddProgressiveStageDataCallback InDelegate);
};

class FBridgeDragDrop
{
public:
    static UE_API void Initialize();
    static UE_API TSharedPtr<FBridgeDragDropImpl> Instance;
};

#undef UE_API
