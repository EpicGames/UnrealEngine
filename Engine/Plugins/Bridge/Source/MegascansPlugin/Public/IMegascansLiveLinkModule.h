// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Modules/ModuleManager.h"




#define MS_MODULE_NAME TEXT("MegascansPlugin")


class IMegascansLiveLinkModule : public IModuleInterface
{


	

public:

	static inline IMegascansLiveLinkModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IMegascansLiveLinkModule>(MS_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(MS_MODULE_NAME);
	}
};

