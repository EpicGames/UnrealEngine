// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MSSettings.generated.h"

struct FPropertyChangedEvent;


UCLASS(Config = Editor)
class UMegascansSettings
	: public UObject
	
{
	GENERATED_UCLASS_BODY()
	//GENERATED_BODY()

public:
	/** Populate Foliage Editor with Foliage Types for 3D Plants. */
	UPROPERTY(Config, DisplayName = "Auto-Populate Foliage Painter", EditAnywhere, Category = "MegascansSettings")
		bool bCreateFoliage;	
	

	/** Ask for confirmation when importing more than 10 assets. */
	/*UPROPERTY(Config, DisplayName = "Prompt Before Batch Import", EditAnywhere, Category = "MegascansSettings")
		bool bBatchImportPrompt;*/
	

	/** Apply imported Surface on selected Actors in Editor. */
	UPROPERTY(Config, DisplayName = "Apply to Selection", EditAnywhere, Category = "MegascansSettings")
		bool bApplyToSelection;



};

UCLASS(Config = Editor)
class UMaterialBlendSettings
	: public UObject	
{
	GENERATED_UCLASS_BODY()
		//GENERATED_BODY()

public:
	/** Package name for Material Blend instance. */
	UPROPERTY(Config, DisplayName = "Material Name", EditAnywhere, Category = "MaterialBlendSettings")
		FString BlendedMaterialName;

	/** Destination path for Material Blend instance. */
	UPROPERTY(Config, DisplayName = "Destination Path", EditAnywhere, Category = "MaterialBlendSettings", meta = (ContentDir))
		FDirectoryPath BlendedMaterialPath;
};


UCLASS(Config = Editor)
class UMaterialAssetSettings
	: public UObject

{
	GENERATED_UCLASS_BODY()
		//GENERATED_BODY()

public:
	UPROPERTY(Config, DisplayName = "3D Master Material", EditAnywhere, Category = "MasterMaterialOverrides")
		FString MasterMaterial3d;

	UPROPERTY(Config, DisplayName = "Surface Master Material", EditAnywhere, Category = "MasterMaterialOverrides")
		FString MasterMaterialSurface;

	UPROPERTY(Config, DisplayName = "Plant Master Material", EditAnywhere, Category = "MasterMaterialOverrides")
		FString MasterMaterialPlant;

};

UCLASS(Config = Editor)
class UMaterialPresetsSettings
	: public UObject
	
{
	GENERATED_UCLASS_BODY()


public:
	/** Replace default master material with your own custom master material for all 3D assets. Default material is used if field is left empty. */
	UPROPERTY(Transient, DisplayName = "3D Master Material", EditAnywhere, Category = "MasterMaterialOverrides")		
	TSoftObjectPtr<class UMaterial> MasterMaterial3d;
	//TSoftObjectPtr<class UMaterialInterface> MasterMaterial3d;

	/** Replace default master material with your own custom master material for all Surfaces. Default material is used if field is left empty. */
	UPROPERTY(Transient, DisplayName = "Surface Master Material", EditAnywhere, Category = "MasterMaterialOverrides")
	TSoftObjectPtr<class UMaterial> MasterMaterialSurface;

	/** Replace default master material with your own custom master material for all 3D Plants. Default material is used if field is left empty. */
	UPROPERTY(Transient, DisplayName = "Plant Master Material", EditAnywhere, Category = "MasterMaterialOverrides")
	TSoftObjectPtr<class UMaterial> MasterMaterialPlant;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyThatWillChange) override;
#endif

};
