// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once



#include "Templates/SharedPointer.h"

#define UE_API MEGASCANSPLUGIN_API

class SWindow;
class UMegascansSettings;

class FTabManager;

class MegascansSettingsWindow
{
public:
	
	
	static UE_API void OpenSettingsWindow(/*const TSharedRef<FTabManager>& TabManager*/);
	static UE_API void SaveSettings(const TSharedRef<SWindow>& Window, UMegascansSettings* MegascansSettings);	
	
	
};

#undef UE_API
