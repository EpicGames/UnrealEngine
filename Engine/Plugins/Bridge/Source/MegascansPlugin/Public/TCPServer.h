// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Containers/UnrealString.h"
#include <string>
#include "Containers/Queue.h"
#include "HAL/Runnable.h"

#define UE_API MEGASCANSPLUGIN_API

class FRunnableThread;
class FSocket;
struct FIPv4Endpoint;
struct FScriptContainerElement;

class FTCPServer : public FRunnable
{
public:	

	UE_API FTCPServer();
	UE_API ~FTCPServer();	
	UE_API virtual bool Init() override;	
	UE_API virtual uint32 Run() override;

	virtual void Stop() override
	{
		Stopping = true;
	}

	
	UE_API bool RecvMessage(FSocket *Socket, uint32 DataSize, FString& Message);
	UE_API bool HandleListenerConnectionAccepted(class FSocket *ClientSocket, const FIPv4Endpoint& ClientEndpoint);

	FSocket* ListenerSocket;
	FString LocalHostIP = "127.0.0.1";
	int32 PortNum = 13429; 
	int32 ConnectionTimeout;
	TArray<class FSocket*> Clients;

	UE_API void HandleIncomingSocket(FSocket* IncomingConnection);
	static UE_API TQueue<FString> ImportQueue;

private:	
	TQueue<class FSocket*, EQueueMode::Mpsc> PendingClients;
	bool Stopping;
	FRunnableThread* ClientThread = NULL;
	class FTcpListener *Listener = NULL;	
	TArray<int32> ConnectionTimer;
};

#undef UE_API
